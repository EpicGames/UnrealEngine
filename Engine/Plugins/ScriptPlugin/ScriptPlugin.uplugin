{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Script Plugin",
	"Description" : "An example of a script plugin. This can be used as a starting point when creating your own plugin.",
	"Category" : "Examples",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	
	"Modules" :
	[
		{
			"Name" : "ScriptPlugin",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault",
			"PlatformDenyList" :
			[
				"Linux"
			]
		},
		{
			"Name" : "ScriptEditorPlugin",
			"Type" : "Editor",
			"LoadingPhase" : "Default",
			"PlatformDenyList" :
			[
				"Linux"
			]
		}
	]
}