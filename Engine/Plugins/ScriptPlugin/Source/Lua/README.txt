To enable Lua integration in UE4, follow these steps:

1. Download Lua source code, unzip it and copy the entire lua-5.3.0 folder to Engine/Plugins/ScriptPlugin/Source/Lua/
2. Open Engine/Plugins/ScriptPlugin/Source/Lua/Lua.sln in Visual Studio
3. Build Release Win32 and Release x64 configurations.
4. Regenerate project files with GenerateProjectFiles.bat
5. Open UE4.sln and rebuild your game target (and the editor/engine)