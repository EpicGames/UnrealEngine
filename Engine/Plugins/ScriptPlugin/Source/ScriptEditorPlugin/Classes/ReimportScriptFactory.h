// Copyright Epic Games, Inc. All Rights Reserved.

//=============================================================================
// ReimportTextureFactory
//=============================================================================

#pragma once

#include "CoreMinimal.h"
#include "ScriptFactory.h"
#include "EditorReimportHandler.h"
#include "ReimportScriptFactory.generated.h"

/**
* Script Blueprint re-import factory
*/
UCLASS(MinimalApi, collapsecategories, EarlyAccessPreview)
class UReimportScriptFactory : public UScriptFactory, public FReimportHandler
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TObjectPtr<class UScriptBlueprint> OriginalScript;

	virtual bool ConfigureProperties() override;

	//~ Begin FReimportHandler interface
	virtual bool CanReimport(UObject* Obj, TArray<FString>& OutFilenames) override;
	virtual void SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths) override;
	virtual EReimportResult::Type Reimport(UObject* Obj) override;
	virtual int32 GetPriority() const override;
	//~ End FReimportHandler interface
};



