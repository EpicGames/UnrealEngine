// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"
#include "ScriptBlueprint.h"
#include "ScriptFactory.generated.h"

/**
* Script Blueprint factory
*/
UCLASS(collapsecategories, hidecategories = Object, EarlyAccessPreview)
class SCRIPTEDITORPLUGIN_API UScriptFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(EditAnywhere, Category = ScriptFactory, meta = (AllowAbstract = "", BlueprintBaseOnly = ""))
	TSubclassOf<class UObject> ParentClass;

	//~ Begin UFactory Interface
	virtual bool ConfigureProperties() override;
	virtual bool DoesSupportClass(UClass* Class) override;
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn);
	//~ End UFactory Interface
};
