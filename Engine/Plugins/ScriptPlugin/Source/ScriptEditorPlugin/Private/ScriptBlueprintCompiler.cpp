// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptBlueprintCompiler.h"
#include "ScriptBlueprint.h"
#include "ScriptBlueprintGeneratedClass.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Kismet2/KismetReinstanceUtilities.h"
#include "ScriptContext.h"
#include "ScriptContextComponent.h"
#include "K2Node_FunctionEntry.h"
#include "K2Node_VariableGet.h"
#include "K2Node_CallFunction.h"
#include "GameFramework/Actor.h"
#include "ScriptPluginComponent.h"

///-------------------------------------------------------------

FScriptBlueprintCompiler::FScriptBlueprintCompiler(UScriptBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions)
	: Super(SourceSketch, InMessageLog, InCompilerOptions)
	, NewScriptBlueprintClass(NULL)
	, ContextProperty(NULL)
{
}

FScriptBlueprintCompiler::~FScriptBlueprintCompiler()
{
}

void FScriptBlueprintCompiler::CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& InOldCDO)
{
	Super::CleanAndSanitizeClass(ClassToClean, InOldCDO);

	// Make sure our typed pointer is set
	check(ClassToClean == NewClass && NewScriptBlueprintClass == NewClass);	
	ContextProperty = NULL;
}

void FScriptBlueprintCompiler::CreateClassVariablesFromBlueprint()
{
	Super::CreateClassVariablesFromBlueprint();

	UScriptBlueprint* ScriptBP = ScriptBlueprint();
	UScriptBlueprintGeneratedClass* NewScripClass = CastChecked<UScriptBlueprintGeneratedClass>(NewClass);
	NewScripClass->ScriptProperties.Empty();

	for (FScriptField& Field : ScriptDefinedFields)
	{
		UClass* InnerType = Field.Class;
		if (Field.PropertyClass)
		{
			FName PinCategory;
			FName PinSubCategory = NAME_None;
			if (Field.PropertyClass->IsChildOf(FStrProperty::StaticClass()))
			{
				PinCategory = UEdGraphSchema_K2::PC_String;
			}
			else if (Field.PropertyClass->IsChildOf(FFloatProperty::StaticClass()))
			{
				PinCategory = UEdGraphSchema_K2::PC_Real;
				PinSubCategory = UEdGraphSchema_K2::PC_Float;
			}
			else if (Field.PropertyClass->IsChildOf(FIntProperty::StaticClass()))
			{
				PinCategory = UEdGraphSchema_K2::PC_Int;
			}
			else if (Field.PropertyClass->IsChildOf(FInt64Property::StaticClass()))
			{
				PinCategory = UEdGraphSchema_K2::PC_Int64;
			}
			else if (Field.PropertyClass->IsChildOf(FBoolProperty::StaticClass()))
			{
				PinCategory = UEdGraphSchema_K2::PC_Boolean;
			}
			else if (Field.PropertyClass->IsChildOf(FObjectProperty::StaticClass()))
			{
				PinCategory = UEdGraphSchema_K2::PC_Object;
				// @todo: some scripting extensions (that are strongly typed) can handle this better
				InnerType = UObject::StaticClass();
			}
			if (!PinCategory.IsNone())
			{
				FEdGraphPinType ScriptPinType(PinCategory, PinSubCategory, InnerType, EPinContainerType::None, false, FEdGraphTerminalType());
				FProperty* ScriptProperty = CreateVariable(Field.Name, ScriptPinType);
				if (ScriptProperty)
				{
					ScriptProperty->SetMetaData(TEXT("Category"), *ScriptBP->GetName());
					ScriptProperty->SetPropertyFlags(CPF_BlueprintVisible | CPF_Edit);
					NewScripClass->ScriptProperties.Add(ScriptProperty);
				}
			}
		}
	}

	CreateScriptContextProperty();
}

void FScriptBlueprintCompiler::CreateScriptContextProperty()
{
	// The only case we don't need a script context is if the script class derives form UScriptPluginComponent
	UClass* ContextClass = nullptr;
	if (Blueprint->ParentClass->IsChildOf(AActor::StaticClass()))
	{
		ContextClass = UScriptContextComponent::StaticClass();
	}
	else if (!Blueprint->ParentClass->IsChildOf(UScriptPluginComponent::StaticClass()))
	{
		ContextClass = UScriptContext::StaticClass();
	}

	if (ContextClass)
	{
		FEdGraphPinType ScriptContextPinType(UEdGraphSchema_K2::PC_Object, NAME_None, ContextClass, EPinContainerType::None, false, FEdGraphTerminalType());
		ContextProperty = CastFieldChecked<FObjectProperty>(CreateVariable(TEXT("Generated_ScriptContext"), ScriptContextPinType));
		ContextProperty->SetPropertyFlags(CPF_ContainsInstancedReference | CPF_InstancedReference);
	}
}

void FScriptBlueprintCompiler::CreateFunctionList()
{
	Super::CreateFunctionList();

	if (!Blueprint->ParentClass->IsChildOf(UScriptPluginComponent::StaticClass()))
	{
		for (auto& Field : ScriptDefinedFields)
		{
			if (Field.Class->IsChildOf(UFunction::StaticClass()))
			{
				CreateScriptDefinedFunction(Field);
			}
		}
	}
}

void FScriptBlueprintCompiler::CreateScriptDefinedFunction(FScriptField& Field)
{
	check(ContextProperty);
	
	UScriptBlueprint* ScriptBP = ScriptBlueprint();
	const FString FunctionName = Field.Name.ToString();

	// Create Blueprint Graph which consists of 3 nodes: 'Entry', 'Get Script Context' and 'Call Function'
	// @todo: once we figure out how to get parameter lists for functions we can add suport for that here

	UEdGraph* ScriptFunctionGraph = NewObject<UEdGraph>(ScriptBP, *FString::Printf(TEXT("%s_Graph"), *FunctionName));
	ScriptFunctionGraph->Schema = UEdGraphSchema_K2::StaticClass();
	ScriptFunctionGraph->SetFlags(RF_Transient);
	
	FKismetFunctionContext* FunctionContext = CreateFunctionContext();
	FunctionContext->SourceGraph = ScriptFunctionGraph;
	FunctionContext->bCreateDebugData = false;

	UK2Node_FunctionEntry* EntryNode = SpawnIntermediateNode<UK2Node_FunctionEntry>(NULL, ScriptFunctionGraph);
	EntryNode->CustomGeneratedFunctionName = Field.Name;
	EntryNode->AllocateDefaultPins();

	UK2Node_VariableGet* GetVariableNode = SpawnIntermediateNode<UK2Node_VariableGet>(NULL, ScriptFunctionGraph);
	GetVariableNode->VariableReference.SetSelfMember(ContextProperty->GetFName());
	GetVariableNode->AllocateDefaultPins();

	
	UK2Node_CallFunction* CallFunctionNode = SpawnIntermediateNode<UK2Node_CallFunction>(NULL, ScriptFunctionGraph);
	CallFunctionNode->FunctionReference.SetExternalMember(TEXT("CallScriptFunction"), ContextProperty->PropertyClass);
	CallFunctionNode->AllocateDefaultPins();
	UEdGraphPin* FunctionNamePin = CallFunctionNode->FindPinChecked(TEXT("FunctionName"));
	FunctionNamePin->DefaultValue = FunctionName;

	// Link nodes together
	UEdGraphPin* ExecPin = Schema->FindExecutionPin(*EntryNode, EGPD_Output);
	UEdGraphPin* GetVariableOutPin = GetVariableNode->FindPinChecked(ContextProperty->GetFName());
	UEdGraphPin* CallFunctionPin = Schema->FindExecutionPin(*CallFunctionNode, EGPD_Input);
	UEdGraphPin* FunctionTargetPin = CallFunctionNode->FindPinChecked(TEXT("self"));
	ExecPin->MakeLinkTo(CallFunctionPin);
	GetVariableOutPin->MakeLinkTo(FunctionTargetPin);
}

void FScriptBlueprintCompiler::FinishCompilingClass(UClass* Class)
{
	UScriptBlueprint* ScriptBP = ScriptBlueprint();

	UScriptBlueprintGeneratedClass* ScriptClass = CastChecked<UScriptBlueprintGeneratedClass>(Class);
	ScriptClass->SourceCode = ScriptBP->SourceCode;
	ScriptClass->ByteCode = ScriptBP->ByteCode;

	// Allow Blueprint Components to be used in Blueprints
	if (ScriptBP->ParentClass->IsChildOf(UScriptPluginComponent::StaticClass()) && Class != ScriptBP->SkeletonGeneratedClass)
	{
		Class->SetMetaData(TEXT("BlueprintSpawnableComponent"), TEXT("true"));
	}

	Super::FinishCompilingClass(Class);

	// Ff context property has been created, create a DSO and set it on the CDO
	if (ContextProperty)
	{
		UObject* CDO = Class->GetDefaultObject();
		UObject* ContextDefaultSubobject = NewObject<UObject>(CDO, ContextProperty->PropertyClass, "ScriptContext", RF_DefaultSubObject | RF_Public);
		ContextProperty->SetObjectPropertyValue(ContextProperty->ContainerPtrToValuePtr<UObject*>(CDO), ContextDefaultSubobject);
	}
}

void FScriptBlueprintCompiler::PreCompile()
{
	ScriptBlueprint()->UpdateSourceCodeIfChanged();
	ScriptContext.Reset(FScriptContextBase::CreateContext(ScriptBlueprint()->SourceCode, NULL, NULL));
	bool Result = true;
	if (ScriptContext)
	{
		ScriptDefinedFields.Empty();
		ScriptContext->GetScriptDefinedFields(ScriptDefinedFields);
	}
	ContextProperty = NULL;
}

void FScriptBlueprintCompiler::EnsureProperGeneratedClass(UClass*& TargetUClass)
{
	if ( TargetUClass && !( (UObject*)TargetUClass )->IsA(UScriptBlueprintGeneratedClass::StaticClass()) )
	{
		FKismetCompilerUtilities::ConsignToOblivion(TargetUClass, Blueprint->bIsRegeneratingOnLoad);
		TargetUClass = NULL;
	}
}

void FScriptBlueprintCompiler::SpawnNewClass(const FString& NewClassName)
{
	NewScriptBlueprintClass = FindObject<UScriptBlueprintGeneratedClass>(Blueprint->GetOutermost(), *NewClassName);

	if ( NewScriptBlueprintClass == NULL )
	{
		NewScriptBlueprintClass = NewObject<UScriptBlueprintGeneratedClass>(Blueprint->GetOutermost(), FName(*NewClassName), RF_Public | RF_Transactional);
	}
	else
	{
		// Already existed, but wasn't linked in the Blueprint yet due to load ordering issues
		FBlueprintCompileReinstancer::Create(NewScriptBlueprintClass);
	}
	NewClass = NewScriptBlueprintClass;
}
	
void FScriptBlueprintCompiler::OnNewClassSet(UBlueprintGeneratedClass* ClassToUse)
{
	NewScriptBlueprintClass = CastChecked<UScriptBlueprintGeneratedClass>(ClassToUse);
}

bool FScriptBlueprintCompiler::ValidateGeneratedClass(UBlueprintGeneratedClass* Class)
{
	bool SuperResult = Super::ValidateGeneratedClass(Class);
	bool Result = UScriptBlueprint::ValidateGeneratedClass(Class);
	return SuperResult && Result;
}
