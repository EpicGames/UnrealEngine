// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "KismetCompiler.h"
#include "ScriptBlueprint.h"
#include "Templates/Casts.h"
#include "Templates/UniquePtr.h"

/**
* Script Blueprint Compiler
*/
class FScriptBlueprintCompiler : public FKismetCompilerContext
{
protected:

	typedef FKismetCompilerContext Super;

public:
	FScriptBlueprintCompiler(UScriptBlueprint* SourceSketch, FCompilerResultsLog& InMessageLog, const FKismetCompilerOptions& InCompilerOptions);
	virtual ~FScriptBlueprintCompiler();

protected:
	UScriptBlueprint* ScriptBlueprint() const { return Cast<UScriptBlueprint>(Blueprint); }

	// FKismetCompilerContext
	virtual void PreCompile() override;
	virtual void SpawnNewClass(const FString& NewClassName) override;
	virtual void OnNewClassSet(UBlueprintGeneratedClass* ClassToUse) override;
	virtual void CleanAndSanitizeClass(UBlueprintGeneratedClass* ClassToClean, UObject*& InOldCDO) override;
	virtual void EnsureProperGeneratedClass(UClass*& TargetClass) override;
	virtual void CreateClassVariablesFromBlueprint() override;
	virtual void CreateFunctionList() override;
	virtual void FinishCompilingClass(UClass* Class) override;
	virtual bool ValidateGeneratedClass(UBlueprintGeneratedClass* Class) override;
	// End FKismetCompilerContext

protected:

	/**
	 * Creates a script context property for this class (if needed)
	 */
	void CreateScriptContextProperty();

	/**
	 * Creates a Blueprint Graph function definition for script defined function
	 *
	 * @param Field Function exported by script
	 */
	void CreateScriptDefinedFunction(FScriptField& Field);

	/** New script class */
	UScriptBlueprintGeneratedClass* NewScriptBlueprintClass;
	/** Script context */
	TUniquePtr<FScriptContextBase> ScriptContext;
	/** Script-defined properties and functions */
	TArray<FScriptField> ScriptDefinedFields;
	/** Script context property generated for the compiled class */
	FObjectProperty* ContextProperty;
};

