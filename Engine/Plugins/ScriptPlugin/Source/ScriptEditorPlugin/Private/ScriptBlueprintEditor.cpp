// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptBlueprintEditor.h"
#include "SKismetInspector.h"
#include "BlueprintEditorModes.h"
#include "ScriptBlueprint.h"

#define LOCTEXT_NAMESPACE "ScriptEditor"

FScriptBlueprintEditor::FScriptBlueprintEditor()
{
}

FScriptBlueprintEditor::~FScriptBlueprintEditor()
{
	UScriptBlueprint* Blueprint = GetScriptBlueprintObj();
	if ( Blueprint )
	{
		Blueprint->OnChanged().RemoveAll(this);
	}
}

void FScriptBlueprintEditor::InitScriptBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode)
{
	InitBlueprintEditor(Mode, InitToolkitHost, InBlueprints, bShouldOpenInDefaultsMode);
	SetCurrentMode(FBlueprintEditorApplicationModes::BlueprintDefaultsMode);
}

void FScriptBlueprintEditor::AddApplicationMode(FName ModeName, TSharedRef<FApplicationMode> Mode)
{
	if (ModeName == FBlueprintEditorApplicationModes::BlueprintDefaultsMode)
	{
		FBlueprintEditor::AddApplicationMode(ModeName, Mode);
	}
}

UScriptBlueprint* FScriptBlueprintEditor::GetScriptBlueprintObj() const
{
	return Cast<UScriptBlueprint>(GetBlueprintObj());
}

UBlueprint* FScriptBlueprintEditor::GetBlueprintObj() const
{
	auto Blueprint = FBlueprintEditor::GetBlueprintObj();
	//auto ScriptBlueprint = Cast<UScriptBlueprint>(Blueprint);
	//ScriptBlueprint->
	return Blueprint;
}

#undef LOCTEXT_NAMESPACE
