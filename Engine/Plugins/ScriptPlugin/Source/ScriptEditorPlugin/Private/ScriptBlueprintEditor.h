// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor/Kismet/Public/BlueprintEditor.h"

/**
 * Script blueprint editor (extends Blueprint editor)
 */
class FScriptBlueprintEditor : public FBlueprintEditor
{
public:
	FScriptBlueprintEditor();
	virtual ~FScriptBlueprintEditor();

	/** Initializes script editor */
	void InitScriptBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, const TArray<UBlueprint*>& InBlueprints, bool bShouldOpenInDefaultsMode);

	/** Work around for the fact there's no other way to prevent Components Mode and Graph Mode from being entered/created */
	virtual void AddApplicationMode(FName ModeName, TSharedRef<FApplicationMode> Mode) override;

	/** @return The Script blueprint currently being edited in this editor */
	class UScriptBlueprint* GetScriptBlueprintObj() const;

	virtual UBlueprint* GetBlueprintObj() const override;
};
