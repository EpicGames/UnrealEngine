// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "IScriptEditorPlugin.h"
#include "ScriptBlueprintCompiler.h"
#include "KismetCompilerModule.h"
#include "ScriptBlueprintEditor.h"
#include "ScriptEditorPluginLog.h"
#include "AssetTypeActions_Base.h"
#include "Misc/MessageDialog.h"

DEFINE_LOG_CATEGORY(LogScriptEditorPlugin);

/**
* Script blueprint editor actions
*/
class FAssetTypeActions_ScriptClass : public FAssetTypeActions_Base
{
public:
	// IAssetTypeActions Implementation
	virtual FText GetName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_ScriptClass", "Script Class"); }
	virtual FColor GetTypeColor() const override { return FColor(201, 29, 85); }
	virtual UClass* GetSupportedClass() const override { return UScriptBlueprint::StaticClass(); }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Misc; }
	virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>()) override
	{
		EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

		for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt)
		{
			auto Blueprint = Cast<UBlueprint>(*ObjIt);
			if (Blueprint && Blueprint->SkeletonGeneratedClass && Blueprint->GeneratedClass)
			{
				TSharedRef<FScriptBlueprintEditor> NewBlueprintEditor(new FScriptBlueprintEditor());

				TArray<UBlueprint*> Blueprints;
				Blueprints.Add(Blueprint);
				NewBlueprintEditor->InitScriptBlueprintEditor(Mode, EditWithinLevelEditor, Blueprints, true);
			}
			else
			{
				FMessageDialog::Open(EAppMsgType::Ok, 
					FText::FromString(TEXT("Script Blueprint could not be loaded because it derives from an invalid class. Check to make sure the parent class for this blueprint hasn't been removed!")));
			}
		}
	}
};

/**
* Script blueprint module
*/
class FScriptEditorPlugin : public IScriptEditorPlugin, IBlueprintCompiler
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:

	virtual bool CanCompile(const UBlueprint* Blueprint) override;

	virtual void PreCompile(UBlueprint* Blueprint) override;
	virtual void Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results) override;
	virtual void PostCompile(UBlueprint* Blueprint) override;
};

IMPLEMENT_MODULE(FScriptEditorPlugin, ScriptEditorPlugin)

void FScriptEditorPlugin::StartupModule()
{
	// Register asset types
	FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get().RegisterAssetTypeActions(MakeShareable(new FAssetTypeActions_ScriptClass));

	// Register blueprint compiler
	IKismetCompilerInterface& KismetCompilerModule = FModuleManager::LoadModuleChecked<IKismetCompilerInterface>("KismetCompiler");
	KismetCompilerModule.GetCompilers().Add(this);
}

void FScriptEditorPlugin::ShutdownModule()
{
}

bool FScriptEditorPlugin::CanCompile(const UBlueprint* Blueprint)
{
	return Cast<UScriptBlueprint>(Blueprint) != nullptr;
}

void FScriptEditorPlugin::PreCompile(UBlueprint* Blueprint)
{

}

void FScriptEditorPlugin::Compile(UBlueprint* Blueprint, const FKismetCompilerOptions& CompileOptions, FCompilerResultsLog& Results)
{
	if ( UScriptBlueprint* ScriptBlueprint = Cast<UScriptBlueprint>(Blueprint) )
	{
		FScriptBlueprintCompiler Compiler(ScriptBlueprint, Results, CompileOptions);
		Compiler.Compile();
		check(Compiler.NewClass);
	}
}

void FScriptEditorPlugin::PostCompile(UBlueprint* Blueprint)
{

}
