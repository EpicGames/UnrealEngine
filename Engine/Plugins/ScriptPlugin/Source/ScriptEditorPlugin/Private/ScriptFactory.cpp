// Copyright Epic Games, Inc. All Rights Reserved. 

#include "ScriptFactory.h"
#include "GameFramework/Actor.h"
#include "Modules/ModuleManager.h"
#include "ClassViewerModule.h"
#include "Kismet2/SClassPickerDialog.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "ScriptEditorPluginLog.h"
#include "Editor.h"
#include "UnrealEdGlobals.h"
#include "EditorFramework/AssetImportData.h"
#include "ReimportScriptFactory.h"
#include "HAL/FileManager.h"
#include "ScriptBlueprintGeneratedClass.h"
#include "Templates/UniquePtr.h"

UScriptFactory::UScriptFactory(const FObjectInitializer& ObjectInitializer)
	: Super( ObjectInitializer )
{
	SupportedClass = UScriptBlueprint::StaticClass();
	ParentClass = AActor::StaticClass();

	FScriptContextBase::GetSupportedScriptFileFormats(Formats);

	bCreateNew = false;
	bEditorImport = true;
	bText = true;
	bEditAfterNew = true;	
}

bool UScriptFactory::ConfigureProperties()
{
	// Null the parent class so we can check for selection later
	ParentClass = NULL;

	// Load the class viewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::TreeView;
	Options.bShowObjectRootClass = true;
	Options.bIsBlueprintBaseOnly = true;
	Options.bShowUnloadedBlueprints = true;

	const FText TitleText = NSLOCTEXT("EditorFactories", "CreateScriptOptions", "Pick Parent Class");
	UClass* ChosenClass = NULL;
	const bool bPressedOk = SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, UScriptBlueprintGeneratedClass::StaticClass());
	if (bPressedOk)
	{
		ParentClass = ChosenClass;
	}

	return bPressedOk;
}


bool UScriptFactory::DoesSupportClass(UClass* Class)
{
	return Class == UScriptBlueprint::StaticClass();
}

UObject* UScriptFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn)
{
	GEditor->SelectNone(true, true, false);

	UScriptBlueprint* NewBlueprint = nullptr;
	TUniquePtr<FScriptContextBase> ScriptContext(FScriptContextBase::CreateContext(Buffer, nullptr, nullptr));
	if (ScriptContext)
	{
	    NewBlueprint = Cast<UScriptBlueprint>( FindObject<UBlueprint>( InParent, *InName.ToString() ) );
	    if( NewBlueprint != nullptr )
	    {
		    NewBlueprint->Modify();
	    }
	    else
	    {
		    NewBlueprint = CastChecked<UScriptBlueprint>(FKismetEditorUtilities::CreateBlueprint(ParentClass, InParent, InName, BPTYPE_Normal, UScriptBlueprint::StaticClass(), UScriptBlueprintGeneratedClass::StaticClass(), "UScriptFactory"));
		}
		
		NewBlueprint->SourceCode = Buffer;

		NewBlueprint->AssetImportData->Update(CurrentFilename);

		// Need to make sure we compile with the new source code
		FKismetEditorUtilities::CompileBlueprint(NewBlueprint);

		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, NewBlueprint);
	}
	else
	{
		UE_LOGF(LogScriptEditorPlugin, Warning, "Failed to import %ls: could not compile script.", *CurrentFilename);
	}

	return NewBlueprint;
}


/** UReimportScriptFactory */

UReimportScriptFactory::UReimportScriptFactory(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

bool UReimportScriptFactory::ConfigureProperties()
{
	return UFactory::ConfigureProperties();
}

bool UReimportScriptFactory::CanReimport(UObject* Obj, TArray<FString>& OutFilenames)
{
	UScriptBlueprint* ScriptClass = Cast<UScriptBlueprint>(Obj);
	if (ScriptClass)
	{
		ScriptClass->AssetImportData->ExtractFilenames(OutFilenames);
		return true;
	}
	return false;
}

void UReimportScriptFactory::SetReimportPaths(UObject* Obj, const TArray<FString>& NewReimportPaths)
{
	UScriptBlueprint* ScriptClass = Cast<UScriptBlueprint>(Obj);
	if (ScriptClass && ensure(NewReimportPaths.Num() == 1))
	{
		ScriptClass->AssetImportData->UpdateFilenameOnly(NewReimportPaths[0]);
	}
}

/**
* Reimports specified texture from its source material, if the meta-data exists
*/
EReimportResult::Type UReimportScriptFactory::Reimport(UObject* Obj)
{
	UScriptBlueprint* ScriptClass = Cast<UScriptBlueprint>(Obj);
	if (!ScriptClass)
	{
		return EReimportResult::Failed;
	}

	TGuardValue<decltype(OriginalScript)> OriginalScriptGuardValue(OriginalScript, ScriptClass);

	const FString ResolvedSourceFilePath = ScriptClass->AssetImportData->GetFirstFilename();
	if (!ResolvedSourceFilePath.Len())
	{
		return EReimportResult::Failed;
	}

	UE_LOGF(LogScriptEditorPlugin, Log, "Performing atomic reimport of [%ls]", *ResolvedSourceFilePath);

	// Ensure that the file provided by the path exists
	if (IFileManager::Get().FileSize(*ResolvedSourceFilePath) == INDEX_NONE)
	{
		UE_LOGF(LogScriptEditorPlugin, Warning, "Cannot reimport: source file cannot be found.");
		return EReimportResult::Failed;
	}

	bool OutCanceled = false;

	if (ImportObject(ScriptClass->GetClass(), ScriptClass->GetOuter(), *ScriptClass->GetName(), RF_Public | RF_Standalone, ResolvedSourceFilePath, nullptr, OutCanceled) != nullptr)
	{
		UE_LOGF(LogScriptEditorPlugin, Log, "Imported successfully");
		// Try to find the outer package so we can dirty it up
		if (ScriptClass->GetOuter())
		{
			ScriptClass->GetOuter()->MarkPackageDirty();
		}
		else
		{
			ScriptClass->MarkPackageDirty();
		}
	}
	else if(OutCanceled)
	{
		UE_LOGF(LogScriptEditorPlugin, Warning, "-- import canceled");
	}
	else
	{
		UE_LOGF(LogScriptEditorPlugin, Warning, "-- import failed");
	}

	return EReimportResult::Succeeded;
}

int32 UReimportScriptFactory::GetPriority() const
{
	return ImportPriority;
}
