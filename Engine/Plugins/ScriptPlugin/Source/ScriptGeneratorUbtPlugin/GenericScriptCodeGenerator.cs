// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;

namespace ScriptGeneratorUbtPlugin
{
	static class GenericScriptCodeGeneratorStringBuilderExtensinos
	{
		public static StringBuilder AppendGenericWrapperFunctionDeclaration(this StringBuilder builder, UhtClass classObj, string functionName)
		{
			return builder.Append("int32 ").Append(classObj.EngineName).Append('_').Append(functionName).Append("(void* InScriptContext)");
		}

		public static StringBuilder AppendGenericFunctionParamDeclaration(this StringBuilder builder, UhtClass _ /* classObj */, UhtProperty property)
		{
			if (property is UhtObjectPropertyBase)
			{
				builder.Append("UObject* ").Append(property.SourceName).Append(" = nullptr;");
			}
			else
			{
				builder
					.AppendPropertyText(property, UhtPropertyTextType.GenericFunctionArgOrRetVal)
					.Append(' ')
					.Append(property.SourceName)
					.Append(" = ").AppendPropertyText(property, UhtPropertyTextType.GenericFunctionArgOrRetVal)
					.Append("();");
			}
			return builder;
		}

		public static StringBuilder AppendGenericObjectDeclarationFromContext(this StringBuilder builder, UhtClass _ /* classObj */)
		{
			return builder.Append("UObject* Obj = (UObject*)InScriptContext;");
		}

		public static StringBuilder AppendGenericReturnValueHandler(this StringBuilder builder, UhtClass _1 /* classObj */, UhtProperty? _2 /* returnValue */)
		{
			return builder.Append("return 0;");
		}
	}

	internal class GenericScriptCodeGenerator : ScriptCodeGeneratorBase
	{
		public GenericScriptCodeGenerator(IUhtExportFactory factory)
			: base(factory)
		{
		}

		protected override bool CanExportClass(UhtClass classObj)
		{
			if (!base.CanExportClass(classObj))
			{
				return false;
			}

			foreach (UhtType child in classObj.Children)
			{
				if (child is UhtFunction function)
				{
					if (CanExportFunction(classObj, function))
					{
						return true;
					}
				}
				else if (child is UhtProperty property)
				{
					if (CanExportProperty(classObj, property))
					{
						return true;
					}
				}
			}
			return false;
		}

		protected override void ExportClass(StringBuilder builder, UhtClass classObj)
		{
			builder.Append("#pragma once\r\n\r\n");

			List<UhtFunction> functions = classObj.Children.Where(x => x is UhtFunction).Cast<UhtFunction>().Reverse().ToList();

			//ETSTODO - Functions are reversed in the engine
			foreach (UhtFunction function in classObj.Functions.Reverse())
			{
				if (CanExportFunction(classObj, function))
				{
					ExportFunction(builder, classObj, function);
				}
			}
			//foreach (UhtType child in Class.Children)
			//{
			//	if (child is UhtFunction function && CanExportFunction(classObj, Function))
			//	{
			//		ExportFunction(Builder, classObj, function);
			//	}
			//}

			foreach (UhtType child in classObj.Children)
			{
				if (child is UhtProperty property && CanExportProperty(classObj, property))
				{
					ExportProperty(builder, classObj, property);
				}
			}

			if (!classObj.ClassFlags.HasAnyFlags(EClassFlags.Abstract))
			{
				builder.AppendGenericWrapperFunctionDeclaration(classObj, "New").Append("\r\n");
				builder.Append("{\r\n");
				builder.Append("\tUObject* Outer = NULL;\r\n");
				builder.Append("\tFName Name = FName(\"ScriptObject\");\r\n");
				builder.Append("\tUObject* Obj = NewObject<").Append(classObj.SourceName).Append(">(Outer, Name);\r\n");
				builder.Append("\tif (Obj)\r\n\t{\r\n");
				builder.Append("\t\tFScriptObjectReferencer::Get().AddObjectReference(Obj);\r\n");
				builder.Append("\t\t// @todo: Register the object with the script context here\r\n");
				builder.Append("\t}\r\n\treturn 0;\r\n");
				builder.Append("}\r\n\r\n");

				builder.AppendGenericWrapperFunctionDeclaration(classObj, "Destroy").Append("\r\n");
				builder.Append("{\r\n");
				builder.Append('\t').AppendGenericObjectDeclarationFromContext(classObj).Append("\r\n");
				builder.Append("\tif (Obj)\r\n\t{\r\n");
				builder.Append("\t\tFScriptObjectReferencer::Get().RemoveObjectReference(Obj);\r\n");
				builder.Append("\t\t// @todo: Remove the object from the script context here if required\r\n");
				builder.Append("\t}\r\n\treturn 0;\r\n");
				builder.Append("}\r\n\r\n");
			}
		}

		protected void ExportFunction(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{
			builder.AppendGenericWrapperFunctionDeclaration(classObj, function.SourceName).Append("\r\n");
			builder.Append("{\r\n");
			builder.Append('\t').AppendGenericObjectDeclarationFromContext(classObj).Append("\r\n");
			AppendFunctionDispatch(builder, classObj, function);
			builder.Append('\t').AppendGenericReturnValueHandler(classObj, function.ReturnProperty).Append("\r\n");
			builder.Append("}\r\n\r\n");
		}

		protected static void ExportProperty(StringBuilder builder, UhtClass classObj, UhtProperty property)
		{
			// Getter
			builder.AppendGenericWrapperFunctionDeclaration(classObj, $"Get_{property.SourceName}").Append("\r\n");
			builder.Append("{\r\n");
			builder.Append('\t').AppendGenericObjectDeclarationFromContext(classObj).Append("\r\n");
			builder.Append("\tstatic FProperty* Property = FindScriptPropertyHelper(").Append(classObj.SourceName).Append("::StaticClass(), TEXT(\"").Append(property.SourceName).Append("\"));\r\n");
			builder.Append('\t').AppendGenericFunctionParamDeclaration(classObj, property).Append("\r\n");
			builder.Append("\tProperty->CopyCompleteValue(&").Append(property.SourceName).Append(", Property->ContainerPtrToValuePtr<void>(Obj));\r\n");
			builder.Append("\t// @todo: handle property value here\r\n");
			builder.Append("\treturn 0;\r\n");
			builder.Append("}\r\n\r\n");

			// Setter
			builder.AppendGenericWrapperFunctionDeclaration(classObj, $"Set_{property.SourceName}").Append("\r\n");
			builder.Append("{\r\n");
			builder.Append('\t').AppendGenericObjectDeclarationFromContext(classObj).Append("\r\n");
			builder.Append("\tstatic FProperty* Property = FindScriptPropertyHelper(").Append(classObj.SourceName).Append("::StaticClass(), TEXT(\"").Append(property.SourceName).Append("\"));\r\n");
			builder.Append('\t').AppendGenericFunctionParamDeclaration(classObj, property).Append("\r\n");
			builder.Append("\tProperty->CopyCompleteValue(Property->ContainerPtrToValuePtr<void>(Obj), &").Append(property.SourceName).Append(");\r\n");
			builder.Append("\treturn 0;\r\n");
			builder.Append("}\r\n\r\n");
		}

		protected override void Finish(StringBuilder builder, List<UhtClass> classes)
		{
			HashSet<UhtHeaderFile> uniqueHeaders = new();
			foreach (UhtClass classObj in classes)
			{
				uniqueHeaders.Add(classObj.HeaderFile);
			}
			List<string> sortedHeaders = new();
			foreach (UhtHeaderFile headerFile in uniqueHeaders)
			{
				sortedHeaders.Add(headerFile.FilePath);
			}
			sortedHeaders.Sort(StringComparerUE.OrdinalIgnoreCase);
			foreach (string filePath in sortedHeaders)
			{
				string relativePath = Factory.GetPluginShortestIncludePath(filePath);
				builder.Append("#include \"").Append(relativePath).Append("\"\r\n");
			}

			sortedHeaders.Clear();
			foreach (UhtClass classObj in classes)
			{
				sortedHeaders.Add($"{classObj.EngineName}.script.h");
			}
			sortedHeaders.Sort(StringComparerUE.OrdinalIgnoreCase);
			foreach (string filePath in sortedHeaders)
			{
				builder.Append("#include \"").Append(filePath).Append("\"\r\n");
			}
		}
	}
}
