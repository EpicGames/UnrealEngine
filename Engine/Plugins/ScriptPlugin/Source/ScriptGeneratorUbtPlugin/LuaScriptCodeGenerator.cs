// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;

namespace ScriptGeneratorUbtPlugin
{
	static class LuaScriptCodeGeneratorStringBuilderExtensinos
	{
		public static StringBuilder AppendLuaWrapperFunctionDeclaration(this StringBuilder builder, UhtClass classObj, string functionName)
		{
			return builder.Append("int32 ").Append(classObj.EngineName).Append('_').Append(functionName).Append("(lua_State* InScriptContext)");
		}

		public static StringBuilder AppendLuaObjectDeclarationFromContext(this StringBuilder builder, UhtClass classObj)
		{
			return builder.Append("UObject* Obj = (").Append(classObj.SourceName).Append("*)lua_touserdata(InScriptContext, 1);");
		}

		public static StringBuilder AppendLuaReturnValueHandler(this StringBuilder builder, UhtClass _ /* classObj */, UhtProperty? returnValue, string? returnValueName)
		{
			if (returnValue != null)
			{
				if (returnValue is UhtIntProperty)
				{
					builder.Append("lua_pushinteger(InScriptContext, ").Append(returnValueName).Append(");\r\n");
				}
				else if (returnValue is UhtFloatProperty)
				{
					builder.Append("lua_pushnumber(InScriptContext, ").Append(returnValueName).Append(");\r\n");
				}
				else if (returnValue is UhtStrProperty)
				{
					builder.Append("lua_pushstring(InScriptContext, TCHAR_TO_ANSI(*").Append(returnValueName).Append("));\r\n");
				}
				else if (returnValue is UhtNameProperty)
				{
					builder.Append("lua_pushstring(InScriptContext, TCHAR_TO_ANSI(*").Append(returnValueName).Append(".ToString()));\r\n");
				}
				else if (returnValue is UhtBoolProperty)
				{
					builder.Append("lua_pushboolean(InScriptContext, ").Append(returnValueName).Append(");\r\n");
				}
				else if (returnValue is UhtStructProperty structProperty)
				{
					if (structProperty.ScriptStruct.EngineName == "Vector2D")
					{
						builder.Append("FLuaVector2D::Return(InScriptContext, ").Append(returnValueName).Append(");\r\n");
					}
					else if (structProperty.ScriptStruct.EngineName == "Vector")
					{
						builder.Append("FLuaVector::Return(InScriptContext, ").Append(returnValueName).Append(");\r\n");
					}
					else if (structProperty.ScriptStruct.EngineName == "Vector4")
					{
						builder.Append("FLuaVector4::Return(InScriptContext, ").Append(returnValueName).Append(");\r\n");
					}
					else if (structProperty.ScriptStruct.EngineName == "Quat")
					{
						builder.Append("FLuaQuat::Return(InScriptContext, ").Append(returnValueName).Append(");\r\n");
					}
					else if (structProperty.ScriptStruct.EngineName == "LinearColor")
					{
						builder.Append("FLuaLinearColor::Return(InScriptContext, ").Append(returnValueName).Append(");\r\n");
					}
					else if (structProperty.ScriptStruct.EngineName == "Color")
					{
						builder.Append("FLuaLinearColor::Return(InScriptContext, FLinearColor(").Append(returnValueName).Append("));\r\n");
					}
					else if (structProperty.ScriptStruct.EngineName == "Transform")
					{
						builder.Append("FLuaTransform::Return(InScriptContext, ").Append(returnValueName).Append(");\r\n");
					}
					else
					{
						throw new UhtIceException($"Unsupported function return value struct type: {structProperty.ScriptStruct.EngineName}");
					}
				}
				else if (returnValue is UhtObjectPropertyBase)
				{
					builder.Append("lua_pushlightuserdata(InScriptContext, ").Append(returnValueName).Append(");\r\n");
				}
				else
				{
					throw new UhtIceException($"Unsupported function return type: {returnValue.GetType().Name}");
				}
				builder.Append("\treturn 1;");
			}
			else
			{
				builder.Append("return 0;");
			}
			return builder;
		}
	}

	internal class LuaScriptCodeGenerator : ScriptCodeGeneratorBase
	{
		public LuaScriptCodeGenerator(IUhtExportFactory factory)
			: base(factory)
		{
		}

		protected override bool CanExportClass(UhtClass classObj)
		{
			if (!base.CanExportClass(classObj))
			{
				return false;
			}

			for (UhtClass? current = classObj; current != null; current = current.SuperClass)
			{
				foreach (UhtType child in current.Children)
				{
					if (child is UhtFunction function)
					{
						if (CanExportFunction(classObj, function))
						{
							return true;
						}
					}
				}
			}

			foreach (UhtType child in classObj.Children)
			{
				if (child is UhtProperty property)
				{
					if (CanExportProperty(classObj, property))
					{
						return true;
					}
				}
			}
			return false;
		}

		protected override bool CanExportFunction(UhtClass classObj, UhtFunction function)
		{
			if (!base.CanExportFunction(classObj, function))
			{
				return false;
			}

			foreach (UhtType child in function.Children)
			{
				if (child is UhtProperty property)
				{
					if (!IsPropertyTypeSupported(property))
					{
						return false;
					}
				}
			}
			return true;
		}

		protected override bool CanExportProperty(UhtClass classObj, UhtProperty property)
		{
			return property.PropertyFlags.HasAnyFlags(EPropertyFlags.Edit) && IsPropertyTypeSupported(property);
		}

		private static bool IsPropertyTypeSupported(UhtProperty property)
		{
			if (property is UhtStructProperty structProperty)
			{
				return
					structProperty.ScriptStruct.EngineName == "Vector2D" ||
					structProperty.ScriptStruct.EngineName == "Vector" ||
					structProperty.ScriptStruct.EngineName == "Vector4" ||
					structProperty.ScriptStruct.EngineName == "Quat" ||
					structProperty.ScriptStruct.EngineName == "LinearColor" ||
					structProperty.ScriptStruct.EngineName == "Color" ||
					structProperty.ScriptStruct.EngineName == "Transform";
			}
			else if (
				property is UhtLazyObjectPtrProperty ||
				property is UhtSoftObjectProperty ||
				property is UhtSoftClassProperty ||
				property is UhtWeakObjectPtrProperty)
			{
				return false;
			}
			else if (
				property is UhtIntProperty ||
				property is UhtFloatProperty ||
				property is UhtStrProperty ||
				property is UhtNameProperty ||
				property is UhtBoolProperty ||
				property is UhtObjectPropertyBase)
			{
				return true;
			}
			return false;
		}

		protected override void ExportClass(StringBuilder builder, UhtClass classObj)
		{
			builder.Append("#pragma once\r\n\r\n");

			List<UhtFunction> functions = classObj.Children.Where(x => x is UhtFunction).Cast<UhtFunction>().Reverse().ToList();

			//ETSTODO - Functions are reversed in the engine
			for (UhtClass? current = classObj; current != null; current = current.SuperClass)
			{
				foreach (UhtFunction function in current.Functions.Reverse())
				{
					if (CanExportFunction(classObj, function))
					{
						ExportFunction(builder, classObj, function);
					}
				}
			}

			for (UhtClass? current = classObj; current != null; current = current.SuperClass)
			{
				foreach (UhtType child in current.Children)
				{
					if (child is UhtProperty property && CanExportProperty(classObj, property))
					{
						ExportProperty(builder, classObj, property);
					}
				}
			}

			if (!classObj.ClassFlags.HasAnyFlags(EClassFlags.Abstract))
			{
				builder.AppendLuaWrapperFunctionDeclaration(classObj, "New").Append("\r\n");
				builder.Append("{\r\n");
				builder.Append("\tUObject* Outer = (UObject*)lua_touserdata(InScriptContext, 1);\r\n");
				builder.Append("\tFName Name = FName(luaL_checkstring(InScriptContext, 2));\r\n");
				builder.Append("\tUObject* Obj = NewObject<").Append(classObj.SourceName).Append(">(Outer, Name);\r\n");
				builder.Append("\tif (Obj)\r\n\t{\r\n");
				builder.Append("\t\tFScriptObjectReferencer::Get().AddObjectReference(Obj);\r\n");
				builder.Append("\t}\r\n");
				builder.Append("\tlua_pushlightuserdata(InScriptContext, Obj);\r\n");
				builder.Append("\treturn 1;\r\n");
				builder.Append("}\r\n\r\n");

				builder.AppendLuaWrapperFunctionDeclaration(classObj, "Destroy").Append("\r\n");
				builder.Append("{\r\n");
				builder.Append('\t').AppendLuaObjectDeclarationFromContext(classObj).Append("\r\n");
				builder.Append("\tif (Obj)\r\n\t{\r\n");
				builder.Append("\t\tFScriptObjectReferencer::Get().RemoveObjectReference(Obj);\r\n");
				builder.Append("\t}\r\n");
				builder.Append("\treturn 0;\r\n");
				builder.Append("}\r\n\r\n");
			}

			// Class: Equivalent of StaticClass()
			builder.AppendLuaWrapperFunctionDeclaration(classObj, "Class").Append("\r\n");
			builder.Append("{\r\n");
			builder.Append("\tUClass* Class = ").Append(classObj.SourceName).Append("::StaticClass();\r\n");
			builder.Append("\tlua_pushlightuserdata(InScriptContext, Class);\r\n");
			builder.Append("\treturn 1;\r\n");
			builder.Append("}\r\n\r\n");

			// Library
			builder.Append("static const luaL_Reg ").Append(classObj.EngineName).Append("_Lib[] =\r\n");
			builder.Append("{\r\n");
			if (!classObj.ClassFlags.HasAnyFlags(EClassFlags.Abstract))
			{
				builder.Append("\t{ \"New\", ").Append(classObj.EngineName).Append("_New },\r\n");
				builder.Append("\t{ \"Destroy\", ").Append(classObj.EngineName).Append("_Destroy },\r\n");
				builder.Append("\t{ \"Class\", ").Append(classObj.EngineName).Append("_Class },\r\n");
			}

			//ETSTODO - Functions are reversed in the engine
			for (UhtClass? current = classObj; current != null; current = current.SuperClass)
			{
				foreach (UhtFunction function in current.Functions.Reverse())
				{
					if (CanExportFunction(classObj, function))
					{
						builder.Append("\t{ \"").Append(function.SourceName).Append("\", ").Append(classObj.EngineName).Append('_').Append(function.SourceName).Append(" },\r\n");
					}
				}
			}

			for (UhtClass? current = classObj; current != null; current = current.SuperClass)
			{
				foreach (UhtType child in current.Children)
				{
					if (child is UhtProperty property && CanExportProperty(classObj, property))
					{
						builder.Append("\t{ \"Get_").Append(property.SourceName).Append("\", ").Append(classObj.EngineName).Append("_Get_").Append(property.SourceName).Append(" },\r\n");
						builder.Append("\t{ \"Set_").Append(property.SourceName).Append("\", ").Append(classObj.EngineName).Append("_Set_").Append(property.SourceName).Append(" },\r\n");
					}
				}
			}

			builder.Append("\t{ NULL, NULL }\r\n");
			builder.Append("};\r\n\r\n");
		}

		protected void ExportFunction(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{
			UhtClass? functionSuper = null;
			if (function.Outer != null && function.Outer != classObj && function.Outer is UhtClass outerClass && base.CanExportClass(outerClass))
			{
				functionSuper = outerClass;
			}

			builder.AppendLuaWrapperFunctionDeclaration(classObj, function.SourceName).Append("\r\n");
			builder.Append("{\r\n");
			if (functionSuper == null)
			{
				builder.Append('\t').AppendLuaObjectDeclarationFromContext(classObj).Append("\r\n");
				AppendFunctionDispatch(builder, classObj, function);
				string returnValueName = function.ReturnProperty != null ? $"Params.{function.ReturnProperty.SourceName}" : String.Empty;
				builder.Append('\t').AppendLuaReturnValueHandler(classObj, function.ReturnProperty, returnValueName).Append("\r\n");
			}
			else
			{
				builder.Append("\treturn ").Append(functionSuper.EngineName).Append('_').Append(function.SourceName).Append("(InScriptContext);\r\n");
			}
			builder.Append("}\r\n\r\n");
		}

		protected void ExportProperty(StringBuilder builder, UhtClass classObj, UhtProperty property)
		{
			UhtClass? propertySuper = null;
			if (property.Outer != null && property.Outer != classObj && property.Outer is UhtClass outerClass && base.CanExportClass(outerClass))
			{
				propertySuper = outerClass;
			}

			// Getter
			builder.AppendLuaWrapperFunctionDeclaration(classObj, $"Get_{property.SourceName}").Append("\r\n");
			builder.Append("{\r\n");
			if (propertySuper == null)
			{
				builder.Append('\t').AppendLuaObjectDeclarationFromContext(classObj).Append("\r\n");
				builder.Append("\tstatic FProperty* Property = FindScriptPropertyHelper(").Append(classObj.SourceName).Append("::StaticClass(), TEXT(\"").Append(property.SourceName).Append("\"));\r\n");
				builder.Append('\t').AppendPropertyText(property, UhtPropertyTextType.GenericFunctionArgOrRetVal).Append(" PropertyValue;\r\n");
				builder.Append("\tProperty->CopyCompleteValue(&PropertyValue, Property->ContainerPtrToValuePtr<void>(Obj));\r\n");
				builder.Append('\t').AppendLuaReturnValueHandler(classObj, property, "PropertyValue").Append("\r\n");
			}
			else
			{
				builder.Append("\treturn ").Append(propertySuper.EngineName).Append('_').Append("Get_").Append(property.SourceName).Append("(InScriptContext);\r\n");
			}
			builder.Append("}\r\n\r\n");

			// Setter
			builder.AppendLuaWrapperFunctionDeclaration(classObj, $"Set_{property.SourceName}").Append("\r\n");
			builder.Append("{\r\n");
			if (propertySuper == null)
			{
				builder.Append('\t').AppendLuaObjectDeclarationFromContext(classObj).Append("\r\n");
				builder.Append("\tstatic FProperty* Property = FindScriptPropertyHelper(").Append(classObj.SourceName).Append("::StaticClass(), TEXT(\"").Append(property.SourceName).Append("\"));\r\n");
				builder.Append('\t').AppendPropertyText(property, UhtPropertyTextType.GenericFunctionArgOrRetVal).Append(" PropertyValue = ");
				AppendInitializeFunctionDispatchParam(builder, classObj, null, property, 0).Append(";\r\n");
				builder.Append("\tProperty->CopyCompleteValue(Property->ContainerPtrToValuePtr<void>(Obj), &PropertyValue);\r\n");
				builder.Append("\treturn 0;\r\n");
			}
			else
			{
				builder.Append("\treturn ").Append(propertySuper.EngineName).Append('_').Append("Set_").Append(property.SourceName).Append("(InScriptContext);\r\n");
			}
			builder.Append("}\r\n\r\n");
		}

		protected override void Finish(StringBuilder builder, List<UhtClass> classes)
		{
			HashSet<UhtHeaderFile> uniqueHeaders = new();
			foreach (UhtClass classObj in classes)
			{
				uniqueHeaders.Add(classObj.HeaderFile);
			}
			List<string> sortedHeaders = new();
			foreach (UhtHeaderFile headerFile in uniqueHeaders)
			{
				sortedHeaders.Add(headerFile.FilePath);
			}
			sortedHeaders.Sort(StringComparerUE.OrdinalIgnoreCase);
			foreach (string filePath in sortedHeaders)
			{
				string relativePath = Factory.GetPluginShortestIncludePath(filePath);
				builder.Append("#include \"").Append(relativePath).Append("\"\r\n");
			}

			classes.Sort((x, y) => StringComparerUE.OrdinalIgnoreCase.Compare(x.EngineName, y.EngineName));
			sortedHeaders.Clear();
			foreach (UhtClass classObj in classes)
			{
				builder.Append("#include \"").Append(classObj.EngineName).Append(".script.h\"\r\n");
			}
			builder.Append("\r\n");
			builder.Append("void LuaRegisterExportedClasses(lua_State* InScriptContext)\r\n");
			builder.Append("{\r\n");
			foreach(UhtClass classObj in classes)
			{
				builder.Append("\tFLuaUtils::RegisterLibrary(InScriptContext, ").Append(classObj.EngineName).Append("_Lib, \"").Append(classObj.EngineName).Append("\");\r\n");
			}
			builder.Append("}\r\n\r\n");
		}

		protected override StringBuilder AppendInitializeFunctionDispatchParam(StringBuilder builder, UhtClass classObj, UhtFunction? function, UhtProperty property, int propertyIndex)
		{
			if (!property.PropertyFlags.HasAnyFlags(EPropertyFlags.ReturnParm))
			{
				int paramIndex = propertyIndex + 2;
				if (property is UhtIntProperty)
				{
					builder.Append("(luaL_checkint");
				}
				else if (property is UhtFloatProperty)
				{
					builder.Append("(float)(luaL_checknumber");
				}
				else if (property is UhtStrProperty)
				{
					builder.Append("ANSI_TO_TCHAR(luaL_checkstring");
				}
				else if (property is UhtNameProperty)
				{
					builder.Append("FName(luaL_checkstring");
				}
				else if (property is UhtBoolProperty)
				{
					builder.Append("!!(lua_toboolean");
				}
				else if (property is UhtStructProperty structProperty)
				{
					if (structProperty.ScriptStruct.EngineName == "Vector2D")
					{
						builder.Append("(FLuaVector2D::Get");
					}
					else if (structProperty.ScriptStruct.EngineName == "Vector")
					{
						builder.Append("(FLuaVector::Get");
					}
					else if (structProperty.ScriptStruct.EngineName == "Vector4")
					{
						builder.Append("(FLuaVector4::Get");
					}
					else if (structProperty.ScriptStruct.EngineName == "Quat")
					{
						builder.Append("(FLuaQuat::Get");
					}
					else if (structProperty.ScriptStruct.EngineName == "LinearColor")
					{
						builder.Append("(FLuaLinearColor::Get");
					}
					else if (structProperty.ScriptStruct.EngineName == "Color")
					{
						builder.Append("FColor(FLuaLinearColor::Get");
					}
					else if (structProperty.ScriptStruct.EngineName == "Transform")
					{
						builder.Append("(FLuaTransform::Get");
					}
					else
					{
						throw new UhtIceException($"Unsupported function param struct type: {structProperty.ScriptStruct.EngineName}");
					}
				}
				else if (property is UhtClassProperty)
				{
					builder.Append("(UClass*)(lua_touserdata");
				}
				else if (property is UhtObjectPropertyBase)
				{
					builder.Append('(').AppendPropertyText(property, UhtPropertyTextType.GenericFunctionArgOrRetVal).Append(")(lua_touserdata");
				}
				else
				{
					throw new UhtIceException($"Unsupported function param type: {property.GetType().Name}");
				}
				builder.Append("(InScriptContext, ").Append(paramIndex).Append("))");
			}
			else
			{
				base.AppendInitializeFunctionDispatchParam(builder, classObj, function, property, propertyIndex);
			}
			return builder;
		}
	}
}
