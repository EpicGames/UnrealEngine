// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.UHT.Types;
using EpicGames.UHT.Utils;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using UnrealBuildBase;
using UnrealBuildTool;

namespace ScriptGeneratorUbtPlugin
{
	internal abstract class ScriptCodeGeneratorBase
	{
		public readonly IUhtExportFactory Factory;
		public UhtSession Session => Factory.Session;

		public ScriptCodeGeneratorBase(IUhtExportFactory factory)
		{
			Factory = factory;
		}

		/// <summary>
		/// Export all the classes in all the packages
		/// </summary>
		public void Generate()
		{
			DirectoryReference configDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Programs/UnrealBuildTool");
			ConfigHierarchy ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, configDirectory, BuildHostPlatform.Current.Platform);
			ini.GetArray("Plugins", "ScriptSupportedModules", out List<string>? supportedScriptModules);

			// Loop through the packages making sure they should be exported.  Queue the export of the classes
			List<UhtClass> classes = new();
			List<Task?> tasks = new();
			foreach (UhtModule module in Session.Modules)
			{
				if (module.Module.ModuleType != UHTModuleType.EngineRuntime && module.Module.ModuleType != UHTModuleType.GameRuntime)
				{
					continue;
				}

				if (supportedScriptModules != null && !supportedScriptModules.Any(x => String.Compare(x, module.Module.Name, StringComparison.OrdinalIgnoreCase) == 0))
				{
					continue;
				}

				QueueClassExports(module.ScriptPackage, module.ScriptPackage, classes, tasks);
			}

			// Wait for all the classes to export
			Task[]? waitTasks = tasks.Where(x => x != null).Cast<Task>().ToArray();
			if (waitTasks.Length > 0)
			{
				Task.WaitAll(waitTasks);
			}

			// Finish the export process
			Finish(classes);
		}

		/// <summary>
		/// Collect the classes to be exported for the given package and type
		/// </summary>
		/// <param name="package">Package being exported</param>
		/// <param name="type">Type to test for exporting</param>
		/// <param name="classes">Collection of exported classes</param>
		/// <param name="tasks">Collection of queued tasks</param>
		private void QueueClassExports(UhtPackage package, UhtType type, List<UhtClass> classes, List<Task?> tasks)
		{
			if (type is UhtClass classObj)
			{
				if (CanExportClass(classObj))
				{
					classes.Add(classObj);
					tasks.Add(Factory.CreateTask((factory) => { ExportClass(classObj); }));
				}
			}
			foreach (UhtType child in type.Children)
			{
				QueueClassExports(package, child, classes, tasks);
			}
		}

		/// <summary>
		/// Test to see if the given class should be exported
		/// </summary>
		/// <param name="classObj">Class to test</param>
		/// <returns>True if the class should be exported, false if not</returns>
		protected virtual bool CanExportClass(UhtClass classObj)
		{
			return classObj.ClassFlags.HasAnyFlags(EClassFlags.RequiredAPI | EClassFlags.MinimalAPI); // Don't export classes that don't export DLL symbols
		}

		/// <summary>
		/// Test to see if the given function should be exported
		/// </summary>
		/// <param name="classObj">Owning class of the function</param>
		/// <param name="function">Function to test</param>
		/// <returns>True if the function should be exported</returns>
		protected virtual bool CanExportFunction(UhtClass classObj, UhtFunction function)
		{
			// We don't support delegates and non-public functions
			if (function.FunctionFlags.HasAnyFlags(EFunctionFlags.Delegate))
			{
				return false;
			}

			// Reject if any of the parameter types is unsupported yet
			foreach (UhtType child in function.Children)
			{
				if (child is UhtArrayProperty ||
					child is UhtDelegateProperty ||
					child is UhtMulticastDelegateProperty ||
					child is UhtWeakObjectPtrProperty ||
					child is UhtInterfaceProperty)
				{
					return false;
				}
				if (child is UhtProperty property && property.IsStaticArray)
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Test to see if the given property should be exported
		/// </summary>
		/// <param name="classObj">Owning class of the property</param>
		/// <param name="property">Property to test</param>
		/// <returns>True if the property should be exported</returns>
		protected virtual bool CanExportProperty(UhtClass classObj, UhtProperty property)
		{
			// Property must be DLL exported
			if (!classObj.ClassFlags.HasAnyFlags(EClassFlags.RequiredAPI))
			{
				return false;
			}

			// Only public, editable properties can be exported
			if (property.PropertyFlags.HasAnyFlags(EPropertyFlags.NativeAccessSpecifierPrivate | EPropertyFlags.NativeAccessSpecifierProtected) || 
				!property.PropertyFlags.HasAnyFlags(EPropertyFlags.Edit))
			{
				return false;
			}

			// Reject if any of the parameter types is unsupported yet
			if (property.IsStaticArray ||
				property is UhtArrayProperty ||
				property is UhtDelegateProperty ||
				property is UhtMulticastDelegateProperty ||
				property is UhtWeakObjectPtrProperty ||
				property is UhtInterfaceProperty ||
				property is UhtStructProperty)
			{
				return false;
			}
			return true;
		}

		/// <summary>
		/// Export the given class
		/// </summary>
		/// <param name="classObj">Class to export</param>
		private void ExportClass(UhtClass classObj)
		{
			using BorrowStringBuilder borrower = new(StringBuilderCache.Big);
			ExportClass(borrower.StringBuilder, classObj);
			string fileName = Factory.MakePath(classObj.EngineName, ".script.h");
			Factory.CommitOutput(fileName, borrower.StringBuilder);
		}

		private void Finish(List<UhtClass> classes)
		{
			using BorrowStringBuilder borrower = new(StringBuilderCache.Big);
			Finish(borrower.StringBuilder, classes);
			string fileName = Factory.MakePath("GeneratedScriptLibraries", ".inl");
			Factory.CommitOutput(fileName, borrower.StringBuilder);
		}

		protected abstract void ExportClass(StringBuilder builder, UhtClass classObj);

		protected abstract void Finish(StringBuilder builder, List<UhtClass> classes);

		protected virtual StringBuilder AppendInitializeFunctionDispatchParam(StringBuilder builder, UhtClass classObj, UhtFunction? function, UhtProperty property, int propertyIndex)
		{
			if (property is UhtObjectPropertyBase)
			{
				builder.Append("NULL");
			}
			else
			{
				builder.AppendPropertyText(property, UhtPropertyTextType.GenericFunctionArgOrRetVal).Append("()");
			}
			return builder;
		}

		protected virtual StringBuilder AppendFunctionDispatch(StringBuilder builder, UhtClass classObj, UhtFunction function)
		{
			bool hasParamsOrReturnValue = function.Children.Count > 0;
			if (hasParamsOrReturnValue)
			{
				builder.Append("\tstruct FDispatchParams\r\n");
				builder.Append("\t{\r\n");
				foreach (UhtProperty property in function.Children)
				{
					builder.Append("\t\t").AppendPropertyText(property, UhtPropertyTextType.GenericFunctionArgOrRetVal).Append(' ').Append(property.SourceName).Append(";\r\n");
				}
				builder.Append("\t} Params;\r\n");
				int propertyIndex = 0;
				foreach (UhtProperty property in function.Children)
				{
					builder.Append("\tParams.").Append(property.SourceName).Append(" = ");
					AppendInitializeFunctionDispatchParam(builder, classObj, function, property, propertyIndex).Append(";\r\n");
					propertyIndex++;
				}
			}

			builder.Append("\tstatic UFunction* Function = Obj->FindFunctionChecked(TEXT(\"").Append(function.SourceName).Append("\"));\r\n");

			if (hasParamsOrReturnValue)
			{
				builder.Append("\tcheck(Function->ParmsSize == sizeof(FDispatchParams));\r\n");
				builder.Append("\tObj->ProcessEvent(Function, &Params);\r\n");
			}
			else
			{
				builder.Append("\tObj->ProcessEvent(Function, NULL);\r\n");
			}
			return builder;
		}
	}
}
