// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.UHT.Tables;
using EpicGames.UHT.Utils;

namespace ScriptGeneratorUbtPlugin
{
	[UnrealHeaderTool]
    class ScriptGenerator
	{
		[UhtExporter(Name = "ScriptPlugin", Description = "Generic Script Plugin Generator", Options = UhtExporterOptions.Default, ModuleName="ScriptPlugin")]
		private static void ScriptGeneratorExporter(IUhtExportFactory Factory)
		{

			// Make sure this plugin should be run
			if (!Factory.Session.IsPluginEnabled("ScriptPlugin", false))
			{
				return;
			}

			// Based on the WITH_LUA setting, run the proper exporter.
			if (Factory.PluginModule != null)
			{
				int Value;
				if (Factory.PluginModule.TryGetDefine("WITH_LUA", out Value))
				{
					if (Value == 0)
					{
						new GenericScriptCodeGenerator(Factory).Generate();
					}
					else
					{
						new LuaScriptCodeGenerator(Factory).Generate();
					}
				}
			}
		}
	}
}
