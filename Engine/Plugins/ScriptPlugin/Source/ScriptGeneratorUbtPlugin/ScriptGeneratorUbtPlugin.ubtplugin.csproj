<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="ScriptGeneratorUbtPlugin.ubtplugin.csproj.props" Condition="Exists('ScriptGeneratorUbtPlugin.ubtplugin.csproj.props')" />
  <Import Project="$(EngineDir)\Source\Programs\Shared\UnrealEngine.csproj.props" />

  <PropertyGroup>
    <OutputType>Library</OutputType>
    <AppendTargetFrameworkToOutputPath>false</AppendTargetFrameworkToOutputPath>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>ScriptGeneratorUbtPlugin</RootNamespace>
    <AssemblyName>ScriptGeneratorUbtPlugin</AssemblyName>
    <OutputPath>..\..\..\..\Binaries\DotNET\UnrealBuildTool\Plugins\ScriptGeneratorUbtPlugin\</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <Nullable>enable</Nullable>
  </PropertyGroup>
  <ItemGroup>
    <Reference Include="EpicGames.Build">
      <HintPath>$(EngineDir)\Binaries\DotNET\UnrealBuildTool\EpicGames.Build.dll</HintPath>
    </Reference>
    <Reference Include="EpicGames.Core">
      <HintPath>$(EngineDir)\Binaries\DotNET\UnrealBuildTool\EpicGames.Core.dll</HintPath>
    </Reference>
    <Reference Include="EpicGames.UHT">
      <HintPath>$(EngineDir)\Binaries\DotNET\UnrealBuildTool\EpicGames.UHT.dll</HintPath>
    </Reference>
    <Reference Include="UnrealBuildTool">
      <HintPath>$(EngineDir)\Binaries\DotNET\UnrealBuildTool\UnrealBuildTool.dll</HintPath>
    </Reference>
  </ItemGroup>
</Project>