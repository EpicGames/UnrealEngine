// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "ScriptBlueprintGeneratedClass.h"
#include "ScriptBlueprint.generated.h"

/**
 * The Script blueprint generates script-defined classes
 */
UCLASS(BlueprintType, EarlyAccessPreview)
class SCRIPTPLUGIN_API UScriptBlueprint : public UBlueprint
{
	GENERATED_UCLASS_BODY()
public:
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString SourceFilePath_DEPRECATED;

	UPROPERTY()
	TObjectPtr<class UAssetImportData> AssetImportData;
#endif

public:

	/** Generated script bytecode */
	UPROPERTY()
	TArray<uint8> ByteCode;

	/** Script source code. @todo: this should be editor-only */
	UPROPERTY()
	FString SourceCode;
	
#if WITH_EDITORONLY_DATA
	/** Override to ensure we write out the asset import data */
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	virtual void PostLoad() override;
#endif

#if WITH_EDITOR
	virtual UClass* GetBlueprintClass() const override;

	virtual bool SupportedByDefaultBlueprintFactory() const override
	{
		return false;
	}
	// End of UBlueprint interface

	static bool ValidateGeneratedClass(const UClass* InClass);
	
	bool IsCodeDirty() const;
	void UpdateScriptStatus();

	/**
	 * Checks if the source code changed and updates it if so (does not recompile the class)
	 */
	void UpdateSourceCodeIfChanged();
#endif
};