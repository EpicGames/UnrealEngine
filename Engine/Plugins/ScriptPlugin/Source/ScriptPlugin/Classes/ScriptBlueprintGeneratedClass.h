// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlueprintGeneratedClass.h"

#include "ScriptBlueprintGeneratedClass.generated.h"

class UScriptBlueprintGeneratedClass;
class UProperty;

/**
* Script-defined field (variable or function)
*/
struct SCRIPTPLUGIN_API FScriptField
{
	/** Field name */
	FName Name;
	/** Field type */
	UClass* Class;
	/** Property type */
	FFieldClass* PropertyClass;

	FScriptField()
		: Class(nullptr)
		, PropertyClass(nullptr)
	{
	}
	FScriptField(FName InName, UClass* InClass)
		: Name(InName)
		, Class(InClass)
		, PropertyClass(nullptr)
	{
	}
	FScriptField(FName InName, FFieldClass* InPropertyClass)
		: Name(InName)
		, Class(nullptr)
		, PropertyClass(InPropertyClass)
	{
	}
};

/** Script context for this component */
class SCRIPTPLUGIN_API FScriptContextBase
{
public:
	
	virtual ~FScriptContextBase() {}

	/**
	* Creates a script context object
	*/
	static FScriptContextBase* CreateContext(const FString& SourceCode, UScriptBlueprintGeneratedClass* Class, UObject* Owner);

	/**
	* Gets supported script file formats (UFactory formats)
	*/
	static void GetSupportedScriptFileFormats(TArray<FString>& OutFormats);

	/**
	 * Initializes script context given script code
	 * @param Code Script code
	 * @param Owner Owner of this context
	 */
	virtual bool Initialize(const FString& Code, UObject* Owner) = 0;

	/**
	* Sends BeginPlay event to the script
	*/
	virtual void BeginPlay() = 0;

	/**
	* Sends Tick event to the script
	*/
	virtual void Tick(float DeltaTime) = 0;

	/**
	* Sends Destroy event to the script and destroys the script.
	*/
	virtual void Destroy() = 0;

	/**
	* @return true if script defines Tick function
	*/
	virtual bool CanTick() = 0;

	/**
	* Calls arbitrary script function (no arguments) given its name.
	* @param FunctionName Name of the function to call
	*/
	virtual bool CallFunction(const FString&  FunctionName) = 0;

	// Property accessors

	virtual bool SetFloatProperty(const FString& PropertyName, float NewValue) = 0;
	virtual float GetFloatProperty(const FString& PropertyName) = 0;
	virtual bool SetIntProperty(const FString& PropertyName, int32 NewValue) = 0;
	virtual int32 GetIntProperty(const FString& PropertyName) = 0;
	virtual bool SetObjectProperty(const FString& PropertyName, UObject* NewValue) = 0;
	virtual UObject* GetObjectProperty(const FString& PropertyName) = 0;
	virtual bool SetBoolProperty(const FString& PropertyName, bool NewValue) = 0;
	virtual bool GetBoolProperty(const FString& PropertyName) = 0;
	virtual bool SetStringProperty(const FString& PropertyName, const FString& NewValue) = 0;
	virtual FString GetStringProperty(const FString& PropertyName) = 0;

	/**
	* Invokes script function from Blueprint code
	*/
	virtual void InvokeScriptFunction(FFrame& Stack, RESULT_DECL) = 0;

#if WITH_EDITOR
	/**
	* Returns a list of exported fields from script (member variables and functions).
	*/
	virtual void GetScriptDefinedFields(TArray<FScriptField>& OutFields) = 0;
#endif

	// Utilities

	/**
	* Pushes all property values from class instance to script.
	*/
	virtual void PushScriptPropertyValues(UScriptBlueprintGeneratedClass* Class, const UObject* Obj);

	/**
	* Fetches all property values from script to class instance.
	*/
	virtual void FetchScriptPropertyValues(UScriptBlueprintGeneratedClass* Class, UObject* Obj);	
};

/**
* Script generated class
*/
UCLASS(EarlyAccessPreview)
class SCRIPTPLUGIN_API UScriptBlueprintGeneratedClass : public UBlueprintGeneratedClass
{
	GENERATED_UCLASS_BODY()

public:

	/** Generated script bytecode */
	UPROPERTY()
	TArray<uint8> ByteCode;

	/** Script source code. @todo: this should be editor-only */
	UPROPERTY()
	FString SourceCode;

	/** Script-generated properties */
	UPROPERTY()
	TArray<TObjectPtr<UProperty>> ScriptProperties_DEPRECATED;
	TArray<TFieldPath<FProperty>> ScriptProperties;

	virtual void PurgeClass(bool bRecompilingOnLoad) override;

	/**
	* Adds a unique native function mapping
	* @param InName Name of the native function
	* @param InPointer Pointer to the native member function
	*/
	void AddUniqueNativeFunction(const FName& InName, FNativeFuncPtr InPointer);

	/**
	* Removes native function mapping
	* @param InName Name of the native function
	*/
	void RemoveNativeFunction(const FName& InName);

	/**
	* Gets UScriptBlueprintGeneratedClass from class hierarchy
	* @return UScriptBlueprintGeneratedClass or NULL
	*/
	inline static UScriptBlueprintGeneratedClass* GetScriptGeneratedClass(UClass* InClass)
	{
		UScriptBlueprintGeneratedClass* ScriptClass = NULL;
		for (UClass* MyClass = InClass; MyClass && !ScriptClass; MyClass = MyClass->GetSuperClass())
		{
			ScriptClass = Cast<UScriptBlueprintGeneratedClass>(MyClass);
		}
		return ScriptClass;
	}

	virtual void Serialize(FArchive& Ar) override;
};
