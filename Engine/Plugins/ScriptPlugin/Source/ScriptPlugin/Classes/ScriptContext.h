// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "ScriptBlueprintGeneratedClass.h"
#include "ScriptContext.generated.h"

/**
* Script context subobject
*/
UCLASS(DefaultToInstanced, EarlyAccessPreview)
class SCRIPTPLUGIN_API UScriptContext : public UObject
{
	GENERATED_UCLASS_BODY()

	virtual void PostInitProperties() override;
	virtual void BeginDestroy() override;

	UFUNCTION(BlueprintCallable, Category="ScriptContext")
	void CallScriptFunction(FString FunctionName);

protected:

	/** Script context (code) */
	FScriptContextBase* Context;
};