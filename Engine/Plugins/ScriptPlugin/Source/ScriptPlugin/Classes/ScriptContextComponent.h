// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "ScriptBlueprintGeneratedClass.h"
#include "ScriptContextComponent.generated.h"

/** 
 * Script-extendable component class
 */
UCLASS(EarlyAccessPreview)
class SCRIPTPLUGIN_API UScriptContextComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UActorComponent Interface.
	virtual void OnRegister() override;
	virtual void InitializeComponent() override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	virtual void BeginDestroy() override;
	//~ Begin UActorComponent Interface.

	/**
	* Calls a script-defined function (no arguments)
	* @param FunctionName Name of the function to call
	*/
	UFUNCTION(BlueprintCallable, Category = "ScriptContext")
	void CallScriptFunction(FString FunctionName);

protected:

	/** Script context (code) */
	FScriptContextBase* Context;
};

