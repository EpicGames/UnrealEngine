// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ScriptMacros.h"
#include "Components/ActorComponent.h"
#include "ScriptBlueprintGeneratedClass.h"
#include "ScriptPluginComponent.generated.h"

/** 
 * Script-extendable component class
 */
UCLASS(Blueprintable, hidecategories = (Object, ActorComponent), editinlinenew, meta = (BlueprintSpawnableComponent), ClassGroup = Script, Abstract, EarlyAccessPreview)
class SCRIPTPLUGIN_API UScriptPluginComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:

	/**
	* Calls a script-defined function (no arguments)
	* @param FunctionName Name of the function to call
	*/
	UFUNCTION(BlueprintCallable, Category = "Script|Functions")
	virtual bool CallScriptFunction(FString FunctionName);

	//~ Begin UActorComponent Interface.
	virtual void OnRegister() override;
	virtual void InitializeComponent() override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	virtual void OnUnregister() override;
	//~ Begin UActorComponent Interface.

protected:

	/** Script context (code) */
	FScriptContextBase* Context;
};

