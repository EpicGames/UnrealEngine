// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameFramework/Actor.h"
#include "ScriptTestActor.generated.h"

/**
* Test Actor. To be removed at some point.
*/
UCLASS(BlueprintType, MinimalAPI, EarlyAccessPreview, notplaceable)
class AScriptTestActor : public AActor
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(Category = Test, EditAnywhere, BlueprintReadWrite)
	FString TestString;

	UPROPERTY(Category = Test, EditAnywhere, BlueprintReadWrite)
	float TestValue;

	UPROPERTY(Category = Test, EditAnywhere, BlueprintReadWrite)
	bool TestBool;

	UFUNCTION()
	float TestFunction(float InValue, float InFactor, bool bMultiply);
};
