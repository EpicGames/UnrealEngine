// Copyright Epic Games, Inc. All Rights Reserved.

#include "LuaIntegration.h"
#include "ScriptPluginLog.h"
#include "ScriptObjectReferencer.h"

#if WITH_LUA

// Forward declaration - definition is in generated inl function.
void LuaRegisterExportedClasses(lua_State* InScriptContext);

void FLuaUtils::RegisterLibrary(lua_State* LuaState, const luaL_Reg Lib[], const ANSICHAR* LibName)
{
	lua_getglobal(LuaState, LibName);
	if (lua_isnil(LuaState, -1))
	{
		lua_pop(LuaState, 1);
		lua_newtable(LuaState);
	}
	luaL_setfuncs(LuaState, Lib, 0);
	lua_setglobal(LuaState, LibName);
}

bool FLuaUtils::CallFunction(lua_State* LuaState, const ANSICHAR* FunctionName)
{
	if (FunctionName != NULL)
	{
		lua_getglobal(LuaState, FunctionName);
	}

	bool bResult = true;
	const int NumArgs = 0;
	const int NumResults = 0;
	if (lua_pcall(LuaState, NumArgs, NumResults, 0) != 0)
	{
		UE_LOGF(LogScriptPlugin, Warning, "Cannot call Lua function %ls: %ls", ANSI_TO_TCHAR(FunctionName), ANSI_TO_TCHAR(lua_tostring(LuaState, -1)));
		bResult = false;
	}
	return bResult;
}

bool FLuaUtils::DoesFunctionExist(lua_State* LuaState, const char* FunctionName)
{
	bool bExists = true;
	lua_getglobal(LuaState, FunctionName);
	if (!lua_isfunction(LuaState, lua_gettop(LuaState)))
	{
		bExists = false;
	}
	lua_pop(LuaState, 1);
	return bExists;
}

struct FLuaUObject
{
	/**
	 * FindObject
	 *
	 * @param Class
	 * @param Package
	 * @param Name
	 * @param bExactClass
	 */
	static int32 LuaStaticFindObject(lua_State* LuaState)
	{
		UClass* Class = (UClass*)lua_touserdata(LuaState, 1);
		UObject* Package = (UObject*)lua_touserdata(LuaState, 2);
		FString Name = ANSI_TO_TCHAR(luaL_checkstring(LuaState, 3));
		EFindObjectFlags SearchFlags = !!(lua_toboolean(LuaState, 4)) ? EFindObjectFlags::ExactClass : EFindObjectFlags::None;
		UObject* Result = StaticFindObject(Class, Package, *Name, SearchFlags);
		lua_pushlightuserdata(LuaState, Result);
		return 1;
	}

	/**
	* FindObjectFast
	*
	* @param Class
	* @param Package
	* @param Name
	* @param bExactClass
	*/
	static int32 LuaStaticFindObjectFast(lua_State* LuaState)
	{
		UClass* Class = (UClass*)lua_touserdata(LuaState, 1);
		UObject* Package = (UObject*)lua_touserdata(LuaState, 2);
		FString Name = ANSI_TO_TCHAR(luaL_checkstring(LuaState, 3));
		EFindObjectFlags SearchFlags = !!(lua_toboolean(LuaState, 4)) ? EFindObjectFlags::ExactClass : EFindObjectFlags::None;
		UObject* Result = StaticFindObjectFast(Class, Package, *Name, SearchFlags);
		lua_pushlightuserdata(LuaState, Result);
		return 1;
	}

	/**
	* LoadObject
	*
	* @param Class
	* @param Outer
	* @param LongPackageName
	*/
	static int32 LuaStaticLoadObject(lua_State* LuaState)
	{
		UClass* Class = (UClass*)lua_touserdata(LuaState, 1);
		UObject* Outer = (UObject*)lua_touserdata(LuaState, 2);
		FString LongPackageName = ANSI_TO_TCHAR(luaL_checkstring(LuaState, 3));
		UObject* Result = StaticLoadObject(Class, Outer, *LongPackageName);
		lua_pushlightuserdata(LuaState, Result);
		return 1;
	}

	/**
	* IsA
	*
	* @param Object
	* @param Class
	*/
	static int32 LuaIsA(lua_State* LuaState)
	{
		UObject* Obj = (UObject*)lua_touserdata(LuaState, 1);
		UClass* Class = (UClass*)lua_touserdata(LuaState, 2);
		bool bResult = Obj->IsA(Class);
		lua_pushboolean(LuaState, bResult);
		return 1;
	}

	/**
	* GetCurrentLevel
	*
	* @param 'this'
	*/
	static int32 LuaGetCurrentLevel(lua_State* LuaState)
	{
		ULevel* Level = NULL;
		UObject* This = (UObject*)lua_touserdata(LuaState, 1);
		AActor* ThisActor = Cast<AActor>(This);
		if (!ThisActor)
		{
			UActorComponent* ThisComponent = Cast<UActorComponent>(This);
			if (ThisComponent)
			{
				ThisActor = ThisComponent->GetOwner();
			}
		}
		if (ThisActor)
		{
			Level = ThisActor->GetLevel();
		}
		lua_pushlightuserdata(LuaState, Level);
		return 1;
	}

	/**
	* LuaGetName
	*
	* @param Object
	*/
	static int32 LuaGetName(lua_State* LuaState)
	{
		UObject* Object = (UObject*)lua_touserdata(LuaState, 1);
		lua_pushstring(LuaState, TCHAR_TO_ANSI(*Object->GetName()));
		return 1;
	}

	/**
	* LuaGetFullName
	*
	* @param Object
	*/
	static int32 LuaGetFullName(lua_State* LuaState)
	{
		UObject* Object = (UObject*)lua_touserdata(LuaState, 1);
		lua_pushstring(LuaState, TCHAR_TO_ANSI(*Object->GetFullName()));
		return 1;
	}

	/**
	* LuaGetPathName
	*
	* @param Object
	*/
	static int32 LuaGetPathName(lua_State* LuaState)
	{
		UObject* Object = (UObject*)lua_touserdata(LuaState, 1);
		lua_pushstring(LuaState, TCHAR_TO_ANSI(*Object->GetPathName()));
		return 1;
	}

	/**
	* LuaGetOuter
	*
	* @param Object
	*/
	static int32 LuaGetOuter(lua_State* LuaState)
	{
		UObject* Object = (UObject*)lua_touserdata(LuaState, 1);
		lua_pushlightuserdata(LuaState, Object->GetOuter());
		return 1;
	}

	/**
	* LuaIsValid
	*
	* @param Object
	*/
	static int32 LuaIsValid(lua_State* LuaState)
	{
		UObject* Object = (UObject*)lua_touserdata(LuaState, 1);
		lua_pushboolean(LuaState, IsValid(Object));
		return 1;
	}

	/**
	* LuaAddReferencedObject
	*
	* @param Object
	*/
	static int32 LuaAddReferencedObject(lua_State* LuaState)
	{
		UObject* Object = (UObject*)lua_touserdata(LuaState, 1);
		FScriptObjectReferencer::Get().AddObjectReference(Object);
		return 0;
	}

	/**
	* LuaAddReferencedObject
	*
	* @param Object
	*/
	static int32 LuaRemoveReferencedObject(lua_State* LuaState)
	{
		UObject* Object = (UObject*)lua_touserdata(LuaState, 1);
		FScriptObjectReferencer::Get().RemoveObjectReference(Object);
		return 0;
	}

	// Library
	static luaL_Reg UObject_Lib[];
};

luaL_Reg FLuaUObject::UObject_Lib[] =
{
	{ "IsA", FLuaUObject::LuaIsA },
	{ "FindObject", FLuaUObject::LuaStaticFindObject },
	{ "FindObjectFast", FLuaUObject::LuaStaticFindObjectFast },
	{ "LoadObject", FLuaUObject::LuaStaticLoadObject },
	{ "GetCurrentLevel", FLuaUObject::LuaGetCurrentLevel },
	{ "GetName", FLuaUObject::LuaGetName },
	{ "GetFullName", FLuaUObject::LuaGetFullName },
	{ "GetPathName", FLuaUObject::LuaGetPathName },
	{ "GetOuter", FLuaUObject::LuaGetOuter },
	{ "IsValid", FLuaUObject::LuaIsValid },
	{ "AddReferencedObject", FLuaUObject::LuaAddReferencedObject },
	{ "RemoveReferencedObject", FLuaUObject::LuaRemoveReferencedObject },
	{ NULL, NULL }
};

FVector2D FLuaVector2D::Get(lua_State* LuaState, int ParamIndex)
{
	FVector2D Result;

	luaL_checktype(LuaState, ParamIndex, LUA_TTABLE);

	lua_rawgeti(LuaState, ParamIndex, 1);
	int Top = lua_gettop(LuaState);
	Result.X = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 2);
	Top = lua_gettop(LuaState);
	Result.Y = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	return Result;
}

void FLuaVector2D::Return(lua_State *LuaState, const FVector2D& Value)
{
	lua_newtable(LuaState);

	lua_pushnumber(LuaState, Value.X);
	lua_rawseti(LuaState, -2, 1);

	lua_pushnumber(LuaState, Value.Y);
	lua_rawseti(LuaState, -2, 2);
}

FVector FLuaVector::Get(lua_State* LuaState, int ParamIndex)
{
	FVector Result;

	luaL_checktype(LuaState, ParamIndex, LUA_TTABLE);

	lua_rawgeti(LuaState, ParamIndex, 1);
	int Top = lua_gettop(LuaState);
	Result.X = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 2);
	Top = lua_gettop(LuaState);
	Result.Y = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 3);
	Top = lua_gettop(LuaState);
	Result.Z = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	return Result;
}

void FLuaVector::Return(lua_State *LuaState, const FVector& Value)
{
	lua_newtable(LuaState);

	lua_pushnumber(LuaState, Value.X);
	lua_rawseti(LuaState, -2, 1);

	lua_pushnumber(LuaState, Value.Y);
	lua_rawseti(LuaState, -2, 2);

	lua_pushnumber(LuaState, Value.Z);
	lua_rawseti(LuaState, -2, 3);
}

FVector4 FLuaVector4::Get(lua_State *LuaState, int ParamIndex)
{
	FVector4 Result;

	luaL_checktype(LuaState, ParamIndex, LUA_TTABLE);

	lua_rawgeti(LuaState, ParamIndex, 1);
	int Top = lua_gettop(LuaState);
	Result.X = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 2);
	Top = lua_gettop(LuaState);
	Result.Y = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 3);
	Top = lua_gettop(LuaState);
	Result.Z = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 4);
	Top = lua_gettop(LuaState);
	Result.W = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	return Result;
}

void FLuaVector4::Return(lua_State* LuaState, const FVector4& Value)
{
	lua_newtable(LuaState);

	lua_pushnumber(LuaState, Value.X);
	lua_rawseti(LuaState, -2, 1);

	lua_pushnumber(LuaState, Value.Y);
	lua_rawseti(LuaState, -2, 2);

	lua_pushnumber(LuaState, Value.Z);
	lua_rawseti(LuaState, -2, 3);

	lua_pushnumber(LuaState, Value.W);
	lua_rawseti(LuaState, -2, 4);
}

FQuat FLuaQuat::Get(lua_State* LuaState, int ParamIndex)
{
	FQuat Result;

	luaL_checktype(LuaState, ParamIndex, LUA_TTABLE);

	lua_rawgeti(LuaState, ParamIndex, 1);
	int Top = lua_gettop(LuaState);
	Result.X = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 2);
	Top = lua_gettop(LuaState);
	Result.Y = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 3);
	Top = lua_gettop(LuaState);
	Result.Z = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 4);
	Top = lua_gettop(LuaState);
	Result.W = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	return Result;
}

void FLuaQuat::Return(lua_State* LuaState, const FQuat& Quat)
{
	lua_newtable(LuaState);

	lua_pushnumber(LuaState, Quat.X);
	lua_rawseti(LuaState, -2, 1);

	lua_pushnumber(LuaState, Quat.Y);
	lua_rawseti(LuaState, -2, 2);

	lua_pushnumber(LuaState, Quat.Z);
	lua_rawseti(LuaState, -2, 3);

	lua_pushnumber(LuaState, Quat.W);
	lua_rawseti(LuaState, -2, 4);
}

FLinearColor FLuaLinearColor::Get(lua_State *LuaState, int ParamIndex)
{
	FLinearColor Result;

	luaL_checktype(LuaState, ParamIndex, LUA_TTABLE);

	lua_rawgeti(LuaState, ParamIndex, 1);
	int Top = lua_gettop(LuaState);
	Result.R = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 2);
	Top = lua_gettop(LuaState);
	Result.G = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 3);
	Top = lua_gettop(LuaState);
	Result.B = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	lua_rawgeti(LuaState, ParamIndex, 4);
	Top = lua_gettop(LuaState);
	Result.A = (float)lua_tonumber(LuaState, Top);
	lua_pop(LuaState, 1);

	return Result;
}

void FLuaLinearColor::Return(lua_State* LuaState, const FLinearColor& Value)
{
	lua_newtable(LuaState);

	lua_pushnumber(LuaState, Value.R);
	lua_rawseti(LuaState, -2, 1);

	lua_pushnumber(LuaState, Value.G);
	lua_rawseti(LuaState, -2, 2);

	lua_pushnumber(LuaState, Value.B);
	lua_rawseti(LuaState, -2, 3);

	lua_pushnumber(LuaState, Value.A);
	lua_rawseti(LuaState, -2, 4);
}


template <typename T, typename R>
static R GetVectorFromTable(lua_State *LuaState, int Index, const ANSICHAR* Key)
{
	lua_getfield(LuaState, Index, Key);
	int Top = lua_gettop(LuaState);
	if (lua_isnil(LuaState, Top))
	{
		lua_pop(LuaState, 1);
		return R();
	}

	Top = lua_gettop(LuaState);
	return T::Get(LuaState, Top);
}

FTransform FLuaTransform::Get(lua_State *LuaState, int ParamIndex)
{
	luaL_checktype(LuaState, ParamIndex, LUA_TTABLE);

	FQuat Rotation = GetVectorFromTable<FLuaQuat, FQuat>(LuaState, ParamIndex, "Rotation");
	FVector Translation = GetVectorFromTable<FLuaVector, FVector>(LuaState, ParamIndex, "Translation");
	FVector Scale3D = GetVectorFromTable<FLuaVector, FVector>(LuaState, ParamIndex, "Scale3D");

	return FTransform(Rotation, Translation, Scale3D);
}

void FLuaTransform::Return(lua_State *LuaState, const FTransform& Value)
{
	lua_newtable(LuaState);
	lua_pushstring(LuaState, "Rotation");
	FLuaQuat::Return(LuaState, Value.GetRotation());
	lua_settable(LuaState, -3);

	lua_pushstring(LuaState, "Translation");
	FLuaVector::Return(LuaState, Value.GetTranslation());
	lua_settable(LuaState, -3);

	lua_pushstring(LuaState, "Scale3D");
	FLuaVector::Return(LuaState, Value.GetScale3D());
	lua_settable(LuaState, -3);
}

static int32 LuaTransformNew(lua_State* LuaState)
{
	FTransform Result = FTransform::Identity;
	FLuaTransform::Return(LuaState, Result);
	return 1;
}

luaL_Reg FTransform_Lib[] =
{
	{ "New", LuaTransformNew },
	{ NULL, NULL }
};

static int32 LuaUnrealLog(lua_State* LuaState)
{
	int ArgCount = lua_gettop(LuaState);
	FString Message;

	for (int ArgIndex = 1; ArgIndex <= ArgCount; ++ArgIndex)
	{
		if (lua_isstring(LuaState, ArgIndex))
		{
			Message += ANSI_TO_TCHAR(lua_tostring(LuaState, ArgIndex));
		}
	}
	UE_LOGF(LogScriptPlugin, Log, "%ls", *Message);

	return 0;
}

static void LuaOverridePrint(lua_State* LuaState)
{
	static const luaL_Reg PrintOverride[] =
	{
		{ "print", LuaUnrealLog },
		{ NULL, NULL }
	};

	lua_getglobal(LuaState, "_G");
	luaL_setfuncs(LuaState, PrintOverride, 0);
	lua_pop(LuaState, 1);
}

static void* LuaAlloc(void *Ud, void *Ptr, size_t OldSize, size_t NewSize)
{
	if (NewSize != 0)
	{
		return FMemory::Realloc(Ptr, NewSize);
	}
	else
	{
		FMemory::Free(Ptr);
		return NULL;
	}
}

static int32 LuaPanic(lua_State *lua_State)
{
	UE_LOGF(LogScriptPlugin, Error, "PANIC: unprotected error in call to Lua API(%ls)", ANSI_TO_TCHAR(lua_tostring(lua_State, -1)));
	return 0;
}


static lua_State* LuaNewState()
{
	lua_State* LuaState = lua_newstate(LuaAlloc, NULL);
	if (LuaState)
	{
		lua_atpanic(LuaState, &LuaPanic);
	}
	return LuaState;
}

static void LuaRegisterUnrealUtilities(lua_State* LuaState)
{
	LuaOverridePrint(LuaState);
	FLuaUtils::RegisterLibrary(LuaState, FLuaUObject::UObject_Lib, "UE");
	FLuaUtils::RegisterLibrary(LuaState, FTransform_Lib, "Transform");
}

FLuaContext::FLuaContext()
: bHasTick(false)
, bHasDestroy(false)
, bHasBeginPlay(false)
, LuaState(NULL)
{
}

FLuaContext::~FLuaContext()
{
	CloseLua();
}

void FLuaContext::CloseLua()
{
	if (LuaState)
	{
		lua_close(LuaState);
		LuaState = NULL;
	}
}

bool FLuaContext::Initialize(const FString& Code, UObject* Owner)
{
	CloseLua();

	bool bResult = false;
	LuaState = LuaNewState();
	luaL_openlibs(LuaState);
	LuaRegisterExportedClasses(LuaState);
	LuaRegisterUnrealUtilities(LuaState);

	if (luaL_loadstring(LuaState, TCHAR_TO_ANSI(*Code)) == 0)
	{
		// Set this pointer
		lua_pushlightuserdata(LuaState, Owner);
		lua_setglobal(LuaState, "this");

		bResult = FLuaUtils::CallFunction(LuaState, NULL);
		if (bResult)
		{
			bHasTick = FLuaUtils::DoesFunctionExist(LuaState, "Tick");
			bHasDestroy = FLuaUtils::DoesFunctionExist(LuaState, "Destroy");
			bHasBeginPlay = FLuaUtils::DoesFunctionExist(LuaState, "BeginPlay");
		}
	}
	else
	{
		UE_LOGF(LogScriptPlugin, Warning, "Cannot load Lua script for %ls: %ls", *Owner->GetPathName(), ANSI_TO_TCHAR(lua_tostring(LuaState, -1)));
	}
	return bResult;
}

void FLuaContext::BeginPlay()
{
	check(LuaState);
	if (bHasBeginPlay)
	{
		FLuaUtils::CallFunction(LuaState, "BeginPlay");
	}
}

void FLuaContext::Tick(float DeltaTime)
{
	check(LuaState && bHasTick);
	FLuaUtils::CallFunction(LuaState, "Tick");
}

void FLuaContext::Destroy()
{
	if (LuaState)
	{
		if (bHasDestroy)
		{
			FLuaUtils::CallFunction(LuaState, "Destroy");
		}

		CloseLua();
	}
}

bool FLuaContext::CanTick()
{
	return bHasTick;
}

bool FLuaContext::CallFunction(const FString& FunctionName)
{
	check(LuaState);

	bool bSuccess = FLuaUtils::DoesFunctionExist(LuaState, TCHAR_TO_ANSI(*FunctionName));
	if (bSuccess)
	{
		bSuccess = FLuaUtils::CallFunction(LuaState, TCHAR_TO_ANSI(*FunctionName));
	}
	else
	{
		UE_LOGF(LogScriptPlugin, Warning, "Failed to call function '%ls' ", *FunctionName);
	}

	return bSuccess;
}

bool FLuaContext::SetFloatProperty(const FString& PropertyName, float NewValue)
{
	check(LuaState);

	bool bExists = false;

	// Does the Property exist, and is it a number?
	lua_getglobal(LuaState, TCHAR_TO_ANSI(*PropertyName));
	if (lua_isnumber(LuaState, lua_gettop(LuaState)))
	{
		bExists = true;
	}

	// Pop after getglobal, since no other lua function is popping the stack
	lua_pop(LuaState, 1);

	if (bExists)
	{
		lua_pushnumber(LuaState, NewValue);
		lua_setglobal(LuaState, TCHAR_TO_ANSI(*PropertyName)); // setglobal pops the stack for us
	}
	else
	{
		UE_LOGF(LogScriptPlugin, Warning, " [LUA] Global '%ls' is NaN. ", *PropertyName);
	}

	return bExists;
}

float FLuaContext::GetFloatProperty(const FString& PropertyName)
{
	check(LuaState);

	float Results = 0;
	lua_getglobal(LuaState, TCHAR_TO_ANSI(*PropertyName));

	if (lua_isnumber(LuaState, lua_gettop(LuaState)))
	{
		Results = (float)lua_tonumber(LuaState, -1); // -1 = top of stack
	}
	else
	{
		UE_LOGF(LogScriptPlugin, Warning, " [LUA] Global '%ls' is NaN. ", *PropertyName);
	}

	lua_pop(LuaState, 1);
	return Results;
}

bool FLuaContext::SetIntProperty(const FString& PropertyName, int32 NewValue)
{
	return SetFloatProperty(PropertyName, (float)NewValue);
}

int32 FLuaContext::GetIntProperty(const FString& PropertyName)
{
	return FMath::TruncToInt(GetFloatProperty(PropertyName));
}

bool FLuaContext::SetObjectProperty(const FString& PropertyName, UObject* NewValue)
{
	check(LuaState);

	bool bExists = false;

	// Does the Property exist, and is it a number?
	lua_getglobal(LuaState, TCHAR_TO_ANSI(*PropertyName));
	if (lua_islightuserdata(LuaState, lua_gettop(LuaState)))
	{
		bExists = true;
	}

	// Pop after getglobal, since no other lua function is popping the stack
	lua_pop(LuaState, 1);

	if (bExists)
	{
		lua_pushlightuserdata(LuaState, NewValue);
		lua_setglobal(LuaState, TCHAR_TO_ANSI(*PropertyName)); // setglobal pops the stack for us
	}
	else
	{
		UE_LOGF(LogScriptPlugin, Warning, " [LUA] Global '%ls' is NaN. ", *PropertyName);
	}

	return bExists;
}

UObject* FLuaContext::GetObjectProperty(const FString& PropertyName)
{
	check(LuaState);

	UObject* Results = NULL;
	lua_getglobal(LuaState, TCHAR_TO_ANSI(*PropertyName));

	if (lua_islightuserdata(LuaState, lua_gettop(LuaState)))
	{
		Results = (UObject*)lua_touserdata(LuaState, -1); // -1 = top of stack
	}
	else
	{
		UE_LOGF(LogScriptPlugin, Warning, " [LUA] Global '%ls' is NaN. ", *PropertyName);
	}

	lua_pop(LuaState, 1);
	return Results;
}

bool FLuaContext::SetBoolProperty(const FString& PropertyName, bool NewValue)
{
	check(LuaState);

	bool bExists = false;

	// Does the Property exist, and is it a number?
	lua_getglobal(LuaState, TCHAR_TO_ANSI(*PropertyName));
	if (lua_isboolean(LuaState, lua_gettop(LuaState)))
	{
		bExists = true;
	}

	// Pop after getglobal, since no other lua function is popping the stack
	lua_pop(LuaState, 1);

	if (bExists)
	{
		lua_pushboolean(LuaState, (int32)NewValue);
		lua_setglobal(LuaState, TCHAR_TO_ANSI(*PropertyName)); // setglobal pops the stack for us
	}
	else
	{
		UE_LOGF(LogScriptPlugin, Warning, " [LUA] Global '%ls' is NaN. ", *PropertyName);
	}

	return bExists;
}

bool FLuaContext::GetBoolProperty(const FString& PropertyName)
{
	check(LuaState);

	bool Results = false;
	lua_getglobal(LuaState, TCHAR_TO_ANSI(*PropertyName));

	if (lua_isboolean(LuaState, lua_gettop(LuaState)))
	{
		Results = !!lua_toboolean(LuaState, -1); // -1 = top of stack
	}
	else
	{
		UE_LOGF(LogScriptPlugin, Warning, " [LUA] Global '%ls' is NaN. ", *PropertyName);
	}

	lua_pop(LuaState, 1);
	return Results;
}

bool FLuaContext::SetStringProperty(const FString& PropertyName, const FString& NewValue)
{
	check(LuaState);

	bool bExists = false;

	lua_getglobal(LuaState, TCHAR_TO_ANSI(*PropertyName));
	if (lua_isstring(LuaState, lua_gettop(LuaState)))
	{
		bExists = true;
	}

	lua_pop(LuaState, 1);

	if (bExists)
	{
		lua_pushstring(LuaState, TCHAR_TO_ANSI(*PropertyName));
		lua_setglobal(LuaState, TCHAR_TO_ANSI(*PropertyName));
	}

	UE_CLOGF(!bExists, LogScriptPlugin, Warning, " [LUA] Global '%ls' is not a String. ", *PropertyName);
	return bExists;
}

FString FLuaContext::GetStringProperty(const FString& PropertyName)
{
	check(LuaState);

	FString Results = "";
	lua_getglobal(LuaState, TCHAR_TO_ANSI(*PropertyName));

	if (lua_isstring(LuaState, lua_gettop(LuaState)))
	{
		Results = (FString)lua_tostring(LuaState, -1);
	}
	else
	{
		UE_LOGF(LogScriptPlugin, Warning, " [LUA] Global '%ls' is not a String. ", *PropertyName);
	}

	lua_pop(LuaState, 1);
	return Results;
}

void FLuaContext::InvokeScriptFunction(FFrame& Stack, RESULT_DECL)
{
	P_FINISH;
	CallFunction(Stack.CurrentNativeFunction->GetName());
}

#if WITH_EDITOR
void FLuaContext::GetScriptDefinedFields(TArray<FScriptField>& OutFields)
{
	check(LuaState);

	lua_pushglobaltable(LuaState);
	lua_pushnil(LuaState);

	while (lua_next(LuaState, -2) != 0)
	{
		int32 KeyType = lua_type(LuaState, -2);
		if (KeyType != LUA_TSTRING)
		{
			lua_pop(LuaState, 1);
			continue;
		}

		int32 ValueType = lua_type(LuaState, -1);
		FString KeyName(ANSI_TO_TCHAR(lua_tostring(LuaState, -2)));
		if (KeyName.Len() == 0 || KeyName.StartsWith("_") || KeyName == TEXT("this"))
		{
			lua_pop(LuaState, 1);
			continue;
		}

		FScriptField PropertyInfo;
		switch (ValueType)
		{
			case LUA_TNUMBER:
				PropertyInfo.Class = FFloatProperty::StaticClass();
				break;
			case LUA_TSTRING:
				PropertyInfo.Class = FStrProperty::StaticClass();
				break;
			case LUA_TBOOLEAN:
				PropertyInfo.Class = FBoolProperty::StaticClass();
				break;
			case LUA_TNIL:
			case LUA_TUSERDATA:
				PropertyInfo.Class = FObjectProperty::StaticClass();
				break;
			case LUA_TFUNCTION:
				if (KeyName.StartsWith(TEXT("Event")))
				{
					PropertyInfo.Class = UFunction::StaticClass();
				}
				break;
		}

		if (PropertyInfo.Class)
		{
			PropertyInfo.Name = FName(*KeyName);
			OutFields.Add(PropertyInfo);
		}

		lua_pop(LuaState, 1);
	}
}
#endif

#endif // WITH_LUA
