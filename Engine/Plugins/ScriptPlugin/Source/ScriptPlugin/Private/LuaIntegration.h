// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_LUA

extern "C"
{
#define LUA_COMPAT_APIINTCASTS 1
#include "lua.h"
#include "lauxlib.h"
#include "lualib.h"
}

class FLuaContext : public FScriptContextBase
{
	bool bHasTick;
	bool bHasDestroy;
	bool bHasBeginPlay;
	lua_State* LuaState;
	
	void CloseLua();

public:

	FLuaContext();
	virtual ~FLuaContext();

	// FScriptContextBase interface
	virtual bool Initialize(const FString& Code, UObject* Owner) override;
	virtual void BeginPlay() override;
	virtual void Tick(float DeltaTime) override;
	virtual void Destroy() override;
	virtual bool CanTick() override;

	virtual bool SetFloatProperty(const FString& PropertyName, float NewValue) override;
	virtual float GetFloatProperty(const FString& PropertyName) override;
	virtual bool SetIntProperty(const FString& PropertyName, int32 NewValue) override;
	virtual int32 GetIntProperty(const FString& PropertyName) override;
	virtual bool SetObjectProperty(const FString& PropertyName, UObject* NewValue) override;
	virtual UObject* GetObjectProperty(const FString& PropertyName) override;
	virtual bool SetBoolProperty(const FString& PropertyName, bool NewValue) override;
	virtual bool GetBoolProperty(const FString& PropertyName) override;
	virtual bool SetStringProperty(const FString& PropertyName, const FString& NewValue) override;
	virtual FString GetStringProperty(const FString& PropertyName) override;

	virtual bool CallFunction(const FString& FunctionName) override;
	virtual void InvokeScriptFunction(FFrame& Stack, RESULT_DECL) override;

#if WITH_EDITOR
	virtual void GetScriptDefinedFields(TArray<FScriptField>& OutFields) override;
#endif
};

struct FLuaUtils
{
	static void RegisterLibrary(lua_State* LuaState, const luaL_Reg Lib[], const ANSICHAR* LibName);
	static bool CallFunction(lua_State* LuaState, const ANSICHAR* FunctionName);
	static bool DoesFunctionExist(lua_State* LuaState, const char* FunctionName);
};

struct FLuaVector2D
{
	static FVector2D Get(lua_State *LuaState, int ParamIndex);
	static void Return(lua_State *LuaState, const FVector2D& Vec);
};

struct FLuaVector
{
	static FVector Get(lua_State *LuaState, int ParamIndex);
	static void Return(lua_State *LuaState, const FVector& Vec);
};

struct FLuaVector4
{
	static FVector4 Get(lua_State *LuaState, int ParamIndex);
	static void Return(lua_State *LuaState, const FVector4& Vec);
};

struct FLuaQuat
{
	static FQuat Get(lua_State *LuaState, int ParamIndex);
	static void Return(lua_State *LuaState, const FQuat& Vec);
};

struct FLuaLinearColor
{
	static FLinearColor Get(lua_State *LuaState, int ParamIndex);
	static void Return(lua_State *LuaState, const FLinearColor& Vec);
};

struct FLuaTransform
{
	static FTransform Get(lua_State *LuaState, int ParamIndex);
	static void Return(lua_State *LuaState, const FTransform& Transform);
};

#endif // WITH_LUA
