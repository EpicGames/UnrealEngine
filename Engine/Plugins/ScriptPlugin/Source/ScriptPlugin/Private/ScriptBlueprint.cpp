// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptBlueprint.h"
#include "ScriptPluginLog.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "UObject/AssetRegistryTagsContext.h"
#if WITH_EDITOR
#include "EditorFramework/AssetImportData.h"
#include "Kismet2/BlueprintEditorUtils.h"
#endif

/////////////////////////////////////////////////////
// UScriptBlueprint

UScriptBlueprint::UScriptBlueprint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	AssetImportData = CreateEditorOnlyDefaultSubobject<UAssetImportData>(TEXT("AssetImportData"));
#endif
}

#if WITH_EDITORONLY_DATA
void UScriptBlueprint::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	if (AssetImportData)
	{
		Context.AddTag( FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden) );
	}

	Super::GetAssetRegistryTags(Context);
}
void UScriptBlueprint::PostLoad()
{
	Super::PostLoad();
	if (!SourceFilePath_DEPRECATED.IsEmpty() && AssetImportData)
	{
		AssetImportData->UpdateFilenameOnly(SourceFilePath_DEPRECATED);
	}
}
#endif

#if WITH_EDITOR
UClass* UScriptBlueprint::GetBlueprintClass() const
{
	return UScriptBlueprintGeneratedClass::StaticClass();
}

bool UScriptBlueprint::ValidateGeneratedClass(const UClass* InClass)
{
	const UScriptBlueprintGeneratedClass* GeneratedClass = Cast<const UScriptBlueprintGeneratedClass>(InClass);
	if ( !ensure(GeneratedClass) )
	{
		return false;
	}
	const UScriptBlueprint* Blueprint = Cast<UScriptBlueprint>(GetBlueprintFromClass(GeneratedClass));
	if ( !ensure(Blueprint) )
	{
		return false;
	}

	return true;
}

bool UScriptBlueprint::IsCodeDirty() const
{
	if (!AssetImportData)
	{
		return true;
	}

	const TArray<FAssetImportInfo::FSourceFile>& Data = AssetImportData->SourceData.SourceFiles;

	if (Data.Num() == 1)
	{
		// Check the timestamp of the file as it is now, and the last imported timestamp
		if (IFileManager::Get().GetTimeStamp(*AssetImportData->GetFirstFilename()) <= Data[0].Timestamp)
		{
			return false;
		}
	}

	return true;
}

void UScriptBlueprint::UpdateScriptStatus()
{
	if (IsCodeDirty())
	{
		FBlueprintEditorUtils::MarkBlueprintAsModified(this);
	}
}

void UScriptBlueprint::UpdateSourceCodeIfChanged()
{
	if (IsCodeDirty() && AssetImportData)
	{
		FString NewScript;
		FString Filename = AssetImportData->GetFirstFilename();
		if (FFileHelper::LoadFileToString(NewScript, *Filename))
		{
			SourceCode = NewScript;
			AssetImportData->Update(Filename);
		}
	}
}
#endif
