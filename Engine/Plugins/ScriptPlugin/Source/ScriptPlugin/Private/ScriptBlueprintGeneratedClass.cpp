// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptBlueprintGeneratedClass.h"
#include "ScriptPluginLog.h"
#include "UObject/CoreObjectVersion.h"
#include "UObject/Package.h"
#include "UObject/UnrealTypePrivate.h"
#if WITH_LUA
#include "LuaIntegration.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptBlueprintGeneratedClass)

/////////////////////////////////////////////////////
// UScriptBlueprintGeneratedClass

UScriptBlueprintGeneratedClass::UScriptBlueprintGeneratedClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UScriptBlueprintGeneratedClass::AddUniqueNativeFunction(const FName& InName, FNativeFuncPtr InPointer)
{
	// Find the function in the class's native function lookup table.
	for (int32 FunctionIndex = 0; FunctionIndex < NativeFunctionLookupTable.Num(); ++FunctionIndex)
	{
		FNativeFunctionLookup& NativeFunctionLookup = NativeFunctionLookupTable[FunctionIndex];
		if (NativeFunctionLookup.Name == InName)
		{
			return;
		}
	}
	new(NativeFunctionLookupTable)FNativeFunctionLookup(InName, InPointer);
}

void UScriptBlueprintGeneratedClass::RemoveNativeFunction(const FName& InName)
{
	// Find the function in the class's native function lookup table.
	for (int32 FunctionIndex = 0; FunctionIndex < NativeFunctionLookupTable.Num(); ++FunctionIndex)
	{
		FNativeFunctionLookup& NativeFunctionLookup = NativeFunctionLookupTable[FunctionIndex];
		if (NativeFunctionLookup.Name == InName)
		{
			NativeFunctionLookupTable.RemoveAt(FunctionIndex);
			return;
		}
	}
}

void UScriptBlueprintGeneratedClass::PurgeClass(bool bRecompilingOnLoad)
{
	Super::PurgeClass(bRecompilingOnLoad);
	ScriptProperties.Empty();
}

void UScriptBlueprintGeneratedClass::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.CustomVer(FCoreObjectVersion::GUID) >= FCoreObjectVersion::FProperties)
	{
		Ar << ScriptProperties;
	}
#if WITH_EDITORONLY_DATA
	else if (Ar.IsLoading())
	{
		for (UProperty* Property : ScriptProperties_DEPRECATED)
		{
			ScriptProperties.Add(Property);
		}
	}
#endif
}

void FScriptContextBase::PushScriptPropertyValues(UScriptBlueprintGeneratedClass* Class, const UObject* Obj)
{
	// @todo: optimize this
	for (TFieldPath<FProperty>& Property : Class->ScriptProperties)
	{
		if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property.Get()))
		{
			float Value = FloatProperty->GetFloatingPointPropertyValue(Property->ContainerPtrToValuePtr<float>(Obj));
			SetFloatProperty(Property->GetName(), Value);
		}
		else if (FIntProperty* IntProperty = CastField<FIntProperty>(Property.Get()))
		{
			int32 Value = IntProperty->GetSignedIntPropertyValue(Property->ContainerPtrToValuePtr<int32>(Obj));
			SetIntProperty(Property->GetName(), Value);
		}
		else if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property.Get()))
		{
			bool Value = BoolProperty->GetPropertyValue(Property->ContainerPtrToValuePtr<void>(Obj));
			SetBoolProperty(Property->GetName(), Value);
		}
		else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property.Get()))
		{
			UObject* Value = ObjectProperty->GetObjectPropertyValue(Property->ContainerPtrToValuePtr<UObject*>(Obj));
			SetObjectProperty(Property->GetName(), Value);
		}
		else if (FStrProperty* StringProperty = CastField<FStrProperty>(Property.Get()))
		{
			FString Value = StringProperty->GetPropertyValue(Property->ContainerPtrToValuePtr<UObject*>(Obj));
			SetStringProperty(Property->GetName(), Value);
		}
	}
}

void FScriptContextBase::FetchScriptPropertyValues(UScriptBlueprintGeneratedClass* Class, UObject* Obj)
{
	// @todo: optimize this
	for (TFieldPath<FProperty>& Property: Class->ScriptProperties)
	{
		if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property.Get()))
		{
			float Value = GetFloatProperty(Property->GetName());
			FloatProperty->SetFloatingPointPropertyValue(Property->ContainerPtrToValuePtr<float>(Obj), Value);
		}
		else if (FIntProperty* IntProperty = CastField<FIntProperty>(Property.Get()))
		{
			int32 Value = GetIntProperty(Property->GetName());
			IntProperty->SetPropertyValue(Property->ContainerPtrToValuePtr<int32>(Obj), Value);
		}
		else if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property.Get()))
		{
			bool Value = GetBoolProperty(Property->GetName());
			BoolProperty->SetPropertyValue(Property->ContainerPtrToValuePtr<float>(Obj), Value);
		}
		else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property.Get()))
		{
			UObject* Value = GetObjectProperty(Property->GetName());
			ObjectProperty->SetObjectPropertyValue(Property->ContainerPtrToValuePtr<UObject*>(Obj), Value);
		}
		else if (FStrProperty* StringProperty = CastField<FStrProperty>(Property.Get()))
		{
			FString Value = GetStringProperty(Property->GetName());
			StringProperty->SetPropertyValue(Property->ContainerPtrToValuePtr<FString>(Obj), Value);
		}
	}
}

FScriptContextBase* FScriptContextBase::CreateContext(const FString& SourceCode, UScriptBlueprintGeneratedClass* Class, UObject* Owner)
{
	FScriptContextBase* NewContext = NULL;
#if WITH_LUA
	NewContext = new FLuaContext();
#endif
	if (NewContext)
	{
		if (NewContext->Initialize(SourceCode, Owner))
		{
			// Push values set by CDO
			if (Class && Owner)
			{
				NewContext->PushScriptPropertyValues(Class, Owner);
			}
		}
		else
		{
			delete NewContext;
			NewContext = NULL;
		}
	}
	return NewContext;
}

void FScriptContextBase::GetSupportedScriptFileFormats(TArray<FString>& OutFormats)
{
#if WITH_LUA
	OutFormats.Add(TEXT("lua;Script"));
#endif
}
