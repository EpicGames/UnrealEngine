// Copyright Epic Games, Inc. All Rights Reserved. 

#include "ScriptContext.h"
#include "ScriptBlueprintGeneratedClass.h"

//////////////////////////////////////////////////////////////////////////

UScriptContext::UScriptContext(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Context = NULL;
}

void UScriptContext::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		auto ContextOwner = GetOuter();
		if (ContextOwner && !ContextOwner->HasAnyFlags(RF_ClassDefaultObject))
		{
			auto ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(ContextOwner->GetClass());
			if (ScriptClass)
			{
				Context = FScriptContextBase::CreateContext(ScriptClass->SourceCode, ScriptClass, ContextOwner);
			}
		}
	}
}

void UScriptContext::BeginDestroy()
{
	if (Context)
	{
		Context->Destroy();
		delete Context;
		Context = NULL;
	}

	Super::BeginDestroy();
}

void UScriptContext::CallScriptFunction(FString FunctionName)
{
	if (Context)
	{
		auto ContextOwner = GetOuter();
		auto ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(ContextOwner->GetClass());
		Context->PushScriptPropertyValues(ScriptClass, ContextOwner);
		Context->CallFunction(FunctionName);
		Context->FetchScriptPropertyValues(ScriptClass, ContextOwner);
	}
}