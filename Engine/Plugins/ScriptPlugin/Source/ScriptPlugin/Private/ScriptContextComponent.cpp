// Copyright Epic Games, Inc. All Rights Reserved. 

#include "ScriptContextComponent.h"
#include "Engine/World.h"

//////////////////////////////////////////////////////////////////////////

UScriptContextComponent::UScriptContextComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = false;
	bAutoActivate = true;
	bWantsInitializeComponent = true;

	Context = nullptr;
}

void UScriptContextComponent::OnRegister()
{
	Super::OnRegister();

	UObject* ContextOwner = GetOuter();
	if (ContextOwner && !HasAnyFlags(RF_ClassDefaultObject) && !ContextOwner->HasAnyFlags(RF_ClassDefaultObject))
	{
		UScriptBlueprintGeneratedClass* ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(ContextOwner->GetClass());
		if (ScriptClass && GetWorld() && GetWorld()->WorldType != EWorldType::Editor)
		{
			Context = FScriptContextBase::CreateContext(ScriptClass->SourceCode, ScriptClass, ContextOwner);
			if (!Context || !Context->CanTick())
			{
				bAutoActivate = false;
				PrimaryComponentTick.bCanEverTick = false;
			}
		}
	}
}

void UScriptContextComponent::InitializeComponent()
{
	Super::InitializeComponent();

	if (Context)
	{
		UObject* ContextOwner = GetOuter();
		UScriptBlueprintGeneratedClass* ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(ContextOwner->GetClass());
		check(ScriptClass);
		Context->PushScriptPropertyValues(ScriptClass, ContextOwner);
		Context->BeginPlay();
		Context->FetchScriptPropertyValues(ScriptClass, ContextOwner);
	}
}

void UScriptContextComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);	
	if (Context)
	{
		UObject* ContextOwner = GetOuter();
		UScriptBlueprintGeneratedClass* ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(ContextOwner->GetClass());
		check(ScriptClass);
		Context->PushScriptPropertyValues(ScriptClass, ContextOwner);
		Context->Tick(DeltaTime);
		Context->FetchScriptPropertyValues(ScriptClass, ContextOwner);
	}
};

void UScriptContextComponent::BeginDestroy()
{
	if (Context)
	{
		Context->Destroy();
		delete Context;
		Context = nullptr;
	}

	Super::BeginDestroy();
}

void UScriptContextComponent::CallScriptFunction(FString FunctionName)
{	
	if (Context)
	{
		UObject* ContextOwner = GetOuter();
		UScriptBlueprintGeneratedClass* ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(ContextOwner->GetClass());
		check(ScriptClass);
		Context->PushScriptPropertyValues(ScriptClass, ContextOwner);
		Context->CallFunction(FunctionName);
		Context->FetchScriptPropertyValues(ScriptClass, ContextOwner);
	}
}
