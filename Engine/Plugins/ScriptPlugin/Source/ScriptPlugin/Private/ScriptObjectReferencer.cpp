// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptObjectReferencer.h"

FScriptObjectReferencer* FScriptObjectReferencer::Singleton = NULL;

void FScriptObjectReferencer::Init()
{
	check(Singleton == NULL);
	Singleton = new FScriptObjectReferencer();
}

void FScriptObjectReferencer::Shutdown()
{
	delete Singleton;
	Singleton = NULL;
}

FScriptObjectReferencer& FScriptObjectReferencer::Get()
{
	check(Singleton);
	return *Singleton;
}

void FScriptObjectReferencer::AddObjectReference(UObject* Object)
{
	ScriptCreatedObjects.Add(Object);
}

void FScriptObjectReferencer::RemoveObjectReference(UObject* Object)
{
	ScriptCreatedObjects.Remove(Object);
}

void FScriptObjectReferencer::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AllowEliminatingReferences(false);
	for (auto& Object : ScriptCreatedObjects)
	{
		Collector.AddReferencedObject(Object);
	}
	Collector.AllowEliminatingReferences(true);
}