// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"

class FScriptObjectReferencer : FGCObject
{
	/** Singleton for easy access */
	static FScriptObjectReferencer* Singleton;

	/** List of all objects currently existing in one of the script contexts */
	TArray<TObjectPtr<UObject>> ScriptCreatedObjects;

public:

	/** Initializes the singleton instance */
	static void Init();

	/** Destroys the singleton */
	static void Shutdown();

	/** Access to the singleton instance */
	static FScriptObjectReferencer& Get();

	/**
	* Adds a new object reference
	*
	* @param Object Object reference
	*/
	void AddObjectReference(UObject* Object);

	/**
	* Removes an existing object reference
	*
	* @param Object Object reference
	*/
	void RemoveObjectReference(UObject* Object);

	// FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FScriptObjectReferencer");
	}
};
