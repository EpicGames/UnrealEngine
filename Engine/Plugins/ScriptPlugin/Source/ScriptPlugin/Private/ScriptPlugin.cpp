// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "ScriptPluginLog.h"
#include "ScriptObjectReferencer.h"
#include "IScriptPlugin.h"
#include "UObject/UnrealType.h"

FProperty* FindScriptPropertyHelper(UClass* Class, FName PropertyName)
{
	for (TFieldIterator<FProperty> PropertyIt(Class, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		if (Property->GetFName() == PropertyName)
		{
			return Property;
		}
	}
	return NULL;
}

DEFINE_LOG_CATEGORY(LogScriptPlugin);

class FScriptPlugin : public IScriptPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FScriptPlugin, ScriptPlugin)


void FScriptPlugin::StartupModule()
{
	FScriptObjectReferencer::Init();

}

void FScriptPlugin::ShutdownModule()
{
	FScriptObjectReferencer::Shutdown();
}
