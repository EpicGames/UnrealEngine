// Copyright Epic Games, Inc. All Rights Reserved. 

#include "ScriptPluginComponent.h"
#include "Engine/World.h"

//////////////////////////////////////////////////////////////////////////

UScriptPluginComponent::UScriptPluginComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = false;
	bAutoActivate = true;
	bWantsInitializeComponent = true;

	Context = nullptr;
}

void UScriptPluginComponent::OnRegister()
{
	Super::OnRegister();

	UScriptBlueprintGeneratedClass* ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(GetClass());
	UWorld* MyWorld = (ScriptClass ? GetWorld() : nullptr);
	if (MyWorld && MyWorld->WorldType != EWorldType::Editor)
	{
		Context = FScriptContextBase::CreateContext(ScriptClass->SourceCode, ScriptClass, this);
		if (!Context || !Context->CanTick())
		{
			bAutoActivate = false;
			PrimaryComponentTick.bCanEverTick = false;
		}
	}
}

void UScriptPluginComponent::InitializeComponent()
{
	Super::InitializeComponent();
	if (Context)
	{
		UScriptBlueprintGeneratedClass* ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(GetClass());
		Context->PushScriptPropertyValues(ScriptClass, this);
		Context->BeginPlay();
		Context->FetchScriptPropertyValues(ScriptClass, this);
	}
}

void UScriptPluginComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	if (Context)
	{
		UScriptBlueprintGeneratedClass* ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(GetClass());
		Context->PushScriptPropertyValues(ScriptClass, this);
		Context->Tick(DeltaTime);
		Context->FetchScriptPropertyValues(ScriptClass, this);
	}
};

void UScriptPluginComponent::OnUnregister()
{
	if (Context)
	{
		Context->Destroy();
		delete Context;
		Context = nullptr;
	}

	Super::OnUnregister();
}

bool UScriptPluginComponent::CallScriptFunction(FString FunctionName)
{
	bool bSuccess = false;
	if (Context)
	{
		UScriptBlueprintGeneratedClass* ScriptClass = UScriptBlueprintGeneratedClass::GetScriptGeneratedClass(GetClass());
		Context->PushScriptPropertyValues(ScriptClass, this);
		bSuccess = Context->CallFunction(FunctionName);
		Context->FetchScriptPropertyValues(ScriptClass, this);
	}
	return bSuccess;
}
