// Copyright Epic Games, Inc. All Rights Reserved. 

#include "ScriptTestActor.h"

//////////////////////////////////////////////////////////////////////////

AScriptTestActor::AScriptTestActor(const FObjectInitializer& ObjectInitializer)
	: Super( ObjectInitializer )
{
}

float AScriptTestActor::TestFunction(float InValue, float InFactor, bool bMultiply)
{
	if (bMultiply)
	{
		return InValue * InFactor;
	}
	else
	{
		return InValue / InFactor;
	}
}