// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;
using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class ScriptPlugin : ModuleRules
	{
		public ScriptPlugin(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"SlateCore",
					// ... add other public dependencies that you statically link with here ...
				}
				);

			if (Target.bBuildEditor == true)
			{

				PublicDependencyModuleNames.AddRange(
					new string[] 
					{
						"EditorFramework",
						"UnrealEd", 
					}
				);

			}

			var LuaPath = Path.Combine("..", "Plugins", "ScriptPlugin", "Source", "Lua");				
			var LuaLibDirectory = Path.Combine(LuaPath, "Lib", Target.Platform.ToString(), "Release");
			var LuaLibPath = Path.Combine(LuaLibDirectory, "Lua.lib");
			if (File.Exists(LuaLibPath))
			{					
				PublicDefinitions.Add("WITH_LUA=1");

				// Path to Lua include files
				var IncludePath = Path.GetFullPath(Path.Combine(LuaPath, "Include"));
				PrivateIncludePaths.Add(IncludePath);

				// Lib file
				PublicAdditionalLibraries.Add(LuaLibPath);

//				Log.TraceVerbose("LUA Integration enabled: {0}", IncludePath);
			}
			else
			{
//				Log.TraceVerbose("LUA Integration NOT enabled");
				PublicDefinitions.Add("WITH_LUA=0");
			}
		}
	}
}
